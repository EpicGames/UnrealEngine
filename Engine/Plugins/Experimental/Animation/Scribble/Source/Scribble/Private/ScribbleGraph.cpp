// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScribbleGraph.h"
#include "ScribbleObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScribbleGraph)

#define LOCTEXT_NAMESPACE "ScribbleGraph"

TAutoConsoleVariable<bool> CVarScribbleEnabled(TEXT("Scribble.Enabled"), false, TEXT("enables the scribbling capabilities on supported widgets / editors"));

void FScribbleGraphData::Reset()
{
	ViewOffset = FVector2f::ZeroVector;
	ZoomAmount = 1.0f;
	Nodes.Reset();
	GuidToNode.Reset();
	NotifyChanged();
}

bool FScribbleGraphData::IsEmpty() const
{
	return Nodes.IsEmpty();
}

int32 FScribbleGraphData::NumNodes() const
{
	return Nodes.Num();
}

FScribbleNode* FScribbleGraphData::GetNode(int32 InNodeIndex)
{
	if (Nodes.IsValidIndex(InNodeIndex))
	{
		return Nodes[InNodeIndex].Get();
	}
	return nullptr;
}

const FScribbleNode* FScribbleGraphData::GetNode(int32 InNodeIndex) const
{
	if (Nodes.IsValidIndex(InNodeIndex))
	{
		return Nodes[InNodeIndex].Get();
	}
	return nullptr;
}

TSharedPtr<FScribbleNode> FScribbleGraphData::GetNodePtr(int32 InNodeIndex) const
{
	if (Nodes.IsValidIndex(InNodeIndex))
	{
		return Nodes[InNodeIndex];
	}
	return nullptr;
}

FScribbleNode* FScribbleGraphData::FindNode(const FGuid& InId)
{
	if (const TSharedPtr<FScribbleNode>* Ptr = GuidToNode.Find(InId))
	{
		return Ptr->Get();
	}
	return nullptr;
}

const FScribbleNode* FScribbleGraphData::FindNode(const FGuid& InId) const
{
	if (const TSharedPtr<FScribbleNode>* Ptr = GuidToNode.Find(InId))
	{
		return Ptr->Get();
	}
	return nullptr;
}

TSharedPtr<FScribbleNode> FScribbleGraphData::FindNodePtr(const FGuid& InId) const
{
	if (const TSharedPtr<FScribbleNode>* Ptr = GuidToNode.Find(InId))
	{
		return *Ptr;
	}
	return nullptr;
}

FScribbleNode* FScribbleGraphData::operator[](int32 InNodeIndex)
{
	return GetNode(InNodeIndex);
}

const FScribbleNode* FScribbleGraphData::operator[](int32 InNodeIndex) const
{
	return GetNode(InNodeIndex);
}

FGuid FScribbleGraphData::AddNode(const TSharedPtr<FScribbleNode>& InNode)
{
	if (!InNode)
	{
		return FGuid();
	}
	
	if(InNode->GetGraph() == this)
	{
		return InNode->GetId();
	}

	if (const FScribbleNode* ExistingNode = FindNode(InNode->GetId()))
	{
		return ExistingNode->GetId();
	}

	check(!InNode->WeakGraph.IsValid());

	if (!InNode->Guid.IsValid())
	{
		InNode->Guid = FGuid::NewGuid();
	}

	InNode->WeakGraph = AsWeak();

	Modify();
	Nodes.Add(InNode);
	GuidToNode.Add(InNode->Guid, InNode);

	NotifyChanged();
	return InNode->Guid;
}

bool FScribbleGraphData::RemoveNode(const TSharedPtr<FScribbleNode>& InNode)
{
	if (!InNode.IsValid())
	{
		return false;
	}
	
	Modify();

	const int32 NumRemoved = Nodes.Remove(InNode);
	if (NumRemoved == 0)
	{
		return false;
	}

	InNode->WeakGraph.Reset();
	
	GuidToNode.Remove(InNode->Guid);

	if (NumRemoved > 0)
	{
		NotifyChanged();
	}
	return NumRemoved > 0;
}

bool FScribbleGraphData::RemoveNode(const FGuid& InId)
{
	TSharedPtr<FScribbleNode> Node = FindNodePtr(InId);
	if (!Node)
	{
		return false;
	}
	return RemoveNode(Node);
}

bool FScribbleGraphData::SupportsAnchors() const
{
	return GetCurrentAnchorDelegate.IsBound() && ResolveAnchorDelegate.IsBound();
}

FName FScribbleGraphData::GetCurrentAnchor() const
{
	if (GetCurrentAnchorDelegate.IsBound())
	{
		return GetCurrentAnchorDelegate.Execute();
	}
	return NAME_None;
}

TSharedPtr<FScribbleNode> FScribbleGraphData::GroupNodes(const TArray<TSharedPtr<FScribbleNode>>& InNodes)
{
	int32 NumLineStrips = 0;
	TArray<TSharedPtr<FScribbleNode>> NodesToGroup;
	for (const TSharedPtr<FScribbleNode>& Node : InNodes)
	{
		if (!Node)
		{
			continue;
		}

		if (Node->GetGraph() != this)
		{
			continue;
		}

		// for now we only support lines strips for grouping
		if (Node->GetType() != EScribbleNodeType::LineStrip)
		{
			continue;
		}

		NodesToGroup.Add(Node);

		if (const FLineStripScribbleNode* LineStripScribbleNode = Cast<FLineStripScribbleNode>(Node.Get()))
		{
			NumLineStrips += LineStripScribbleNode->LineStrips.Num();
		}
	}
	
	if (NodesToGroup.Num() < 2)
	{
		return nullptr;
	}

	Modify();
	IncrementChangeBracket();

	TSharedPtr<FLineStripScribbleNode> GroupedNode = MakeShared<FLineStripScribbleNode>();
	GroupedNode->LineStrips.Reserve(NumLineStrips);
	AddNode(GroupedNode);

	const FVector2f GroupedNodePosition = GroupedNode->GetPosition();
	for (const TSharedPtr<FScribbleNode>& Node : NodesToGroup)
	{
		const FLineStripScribbleNode* LineStripScribbleNode = Cast<FLineStripScribbleNode>(Node.Get());
		if (!LineStripScribbleNode)
		{
			continue;
		}

		const FVector2f UngroupedNodePosition = LineStripScribbleNode->GetPosition();
		for (const FScribbleLineStrip& OldLineStrip : LineStripScribbleNode->LineStrips)
		{
			GroupedNode->LineStrips.Add(OldLineStrip);
			FScribbleLineStrip& NewLineStrip = GroupedNode->LineStrips.Last();
			for (FVector2f& Vertex : NewLineStrip.Vertices)
			{
				Vertex = (Vertex + UngroupedNodePosition) - GroupedNodePosition;
			}
		}
	}

	const FBox2f Bounds = GroupedNode->GetContentBounds();
	if (Bounds.bIsValid)
	{
		const FVector2f TopLeft = GroupedNode->GetPosition() + Bounds.GetCenter() - Bounds.GetExtent();
		GroupedNode->OffsetPosition(TopLeft);
	}

	DecrementChangeBracket();
	return GroupedNode;
}

TArray<TSharedPtr<FScribbleNode>> FScribbleGraphData::UngroupNode(const TSharedPtr<FScribbleNode>& InNode)
{
	TArray<TSharedPtr<FScribbleNode>> UngroupedNodes;
	if (!InNode || InNode->GetGraph() != this)
	{
		return UngroupedNodes;
	}

	// for now we only allow ungrouping of line strips
	if (InNode->GetType() != EScribbleNodeType::LineStrip)
	{
		return UngroupedNodes;
	}

	const FLineStripScribbleNode* LineStripScribbleNode = Cast<FLineStripScribbleNode>(InNode.Get());
	if (!LineStripScribbleNode)
	{
		return UngroupedNodes;
	}

	Modify();
	IncrementChangeBracket();

	const FVector2f GroupedNodePosition = LineStripScribbleNode->GetPosition();
	for (const FScribbleLineStrip& LineStrip : LineStripScribbleNode->LineStrips)
	{
		TSharedPtr<FLineStripScribbleNode> UngroupedNode = MakeShared<FLineStripScribbleNode>();
		UngroupedNode->LineStrips.Add(LineStrip);
		AddNode(UngroupedNode);
		const FVector2f UngroupedNodePosition = UngroupedNode->GetPosition();

		FScribbleLineStrip& NewLineStrip = UngroupedNode->LineStrips.Last();
		for (FVector2f& Vertex : NewLineStrip.Vertices)
		{
			Vertex = (Vertex + GroupedNodePosition) - UngroupedNodePosition;
		}

		const FBox2f Bounds = UngroupedNode->GetContentBounds();
		if (Bounds.bIsValid)
		{
			const FVector2f TopLeft = UngroupedNode->GetPosition() + Bounds.GetCenter() - Bounds.GetExtent();
			UngroupedNode->OffsetPosition(TopLeft);
			UngroupedNodes.Add(UngroupedNode);
		}
	}

	DecrementChangeBracket();
	return UngroupedNodes;
}

bool FScribbleGraphData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FScribbleObjectVersion::GUID);
	IncrementChangeBracket();

	Ar << ViewOffset;
	Ar << ZoomAmount;
	
	int32 Num = Nodes.Num();
	Ar << Num;

	if (Ar.IsLoading())
	{
		Nodes.Reset();
		GuidToNode.Reset();

		for (int32 Index = 0; Index < Num; ++Index)
		{
			uint8 EnumAsByte = 0;
			Ar << EnumAsByte;

			TSharedPtr<FScribbleNode> Node;
			switch (static_cast<EScribbleNodeType::Type>(EnumAsByte))
			{
				case EScribbleNodeType::LineStrip:
				{
					Node = MakeShared<FLineStripScribbleNode>();
					break;
				}
				default:
				{
					checkNoEntry();
					break;
				}
			}

			if (!Node)
			{
				continue;
			}

			if (!Node->Serialize(Ar))
			{
				DecrementChangeBracket();
				return false;
			}
			
			Nodes.Add(Node);
			GuidToNode.Add(Node->GetId(), Node);
			Node->WeakGraph = AsWeak();
			NotifyChanged();	
		}
	}
	else
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			uint8 EnumAsByte = Nodes[Index]->GetType();
			Ar << EnumAsByte;
			if (!Nodes[Index]->Serialize(Ar))
			{
				DecrementChangeBracket();
				return false;
			}
		}
	}
	
	DecrementChangeBracket();
	return true;
}

void FScribbleGraphData::IncrementChangeBracket()
{
	ChangeBracket++;
}

void FScribbleGraphData::DecrementChangeBracket()
{
	ChangeBracket = FMath::Max(ChangeBracket - 1, 0);
	NotifyChanged();
}

void FScribbleGraphData::Modify()
{
	OnModifyEvent.Broadcast();
}

void FScribbleGraphData::SetView(const FVector2f& InViewOffset, float InZoomAmount)
{
	if (FMath::IsNearlyEqual(InZoomAmount, ZoomAmount) &&
		InViewOffset.Equals(ViewOffset))
	{
		return;
	}

	Modify();
	ViewOffset = InViewOffset;
	ZoomAmount = InZoomAmount;
}

void FScribbleGraphData::NotifyChanged()
{
	if (ChangeBracket == 0 && AccumulatedChanges > 0)
	{
		AccumulatedChanges = 0;
		OnChangedEvent.Broadcast();
	}
	else
	{
		AccumulatedChanges++;
	}
}

TOptional<FVector2f> FScribbleGraphData::ResolveAnchor(const FName& InName) const
{
	if (ResolveAnchorDelegate.IsBound())
	{
		return ResolveAnchorDelegate.Execute(InName);
	}
	return TOptional<FVector2f>();
}

void FScribbleGraph::Reset()
{
	if (!Data.IsValid())
	{
		Data = MakeShared<FScribbleGraphData>();
	}
	Data->Reset();
}

bool FScribbleGraph::IsValid() const
{
	return Data.IsValid();
}

FScribbleGraphData* FScribbleGraph::GetData()
{
	if (Data.IsValid())
	{
		return Data.Get();
	}
	return nullptr;
}

const FScribbleGraphData* FScribbleGraph::GetData() const
{
	if (Data.IsValid())
	{
		return Data.Get();
	}
	return nullptr;
}

TSharedPtr<FScribbleGraphData> FScribbleGraph::GetOrCreateDataPtr()
{
	if (!Data.IsValid())
	{
		Data = MakeShared<FScribbleGraphData>();
	}
	return Data;
}

bool FScribbleGraph::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FScribbleObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		if (!Data.IsValid())
		{
			Data = MakeShared<FScribbleGraphData>();
		}
		
		bool bContainsData = false;
		Ar << bContainsData;
		if (bContainsData)
		{
			if (!Data->Serialize(Ar))
			{
				return false;
			}
		}
		else
		{
			Data->Reset();
		}
	}
	else if (Ar.IsSaving())
	{
		bool bContainsData = Data.IsValid() && CVarScribbleEnabled.GetValueOnAnyThread();
		Ar << bContainsData;
		if (bContainsData)
		{
			if (!Data->Serialize(Ar))
			{
				return false;
			}
		}
	}
	
	return true;
}

bool FScribbleGraph::Identical(const FScribbleGraph* Other, uint32 PortFlags) const
{
	if (!Other)
	{
		return false;
	}
	if (Data.IsValid() != Other->Data.IsValid())
	{
		return false;
	}
	if (!Data.IsValid())
	{
		return true;
	}

	const TSharedPtr<FScribbleGraphData>& OtherData = Other->Data;

	if (Data->NumNodes() != OtherData->NumNodes())
	{
		return false;
	}
	if (!FMath::IsNearlyEqual(Data->GetZoomAmount(), OtherData->GetZoomAmount()))
	{
		return false;
	}
	if (!(Data->GetViewOffset() - OtherData->GetViewOffset()).IsNearlyZero())
	{
		return false;
	}

	for (int32 NodeIndex = 0; NodeIndex < Data->NumNodes(); NodeIndex++)
	{
		const FScribbleNode* Node = Data->GetNode(NodeIndex);
		const FScribbleNode* OtherNode = OtherData->GetNode(NodeIndex);
		if (!Node || !OtherNode)
		{
			return false;
		}
		if(!Node->Identical(OtherNode, PortFlags))
		{
			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
