// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "ScribbleObjectVersion.h"
#include "UObject/DevObjectVersion.h"

// Unique Scribble Object version id
const FGuid FScribbleObjectVersion::GUID(0xDC49359B, 0x53C04DE7, 0x9126EA88, 0x5E1C5D39);

// Register Scribble custom version with Core
static FDevVersionRegistration GRegisterScribbleObjectVersion(FScribbleObjectVersion::GUID, FScribbleObjectVersion::LatestVersion, TEXT("Dev-Scribble"));

class FScribbleModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
	}
	
	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FScribbleModule, Scribble)
