// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScribbleNode.h"
#include "ScribbleSettings.h"
#include "ScribbleGraph.h"
#include "ScribbleObjectVersion.h"
#if WITH_EDITOR
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScribbleNode)

#define LOCTEXT_NAMESPACE "ScribbleNode"

void FScribbleNode::SetAnchor(const FName& InAnchor)
{
	const FVector2f OldPosition = GetPosition();
	Anchor = InAnchor;
	UpdateRelativePosition(OldPosition);
}

FVector2f FScribbleNode::GetPosition(bool bResolveAnchor) const
{
	if (bResolveAnchor && !Anchor.IsNone())
	{
		if (const FScribbleGraphData* Graph = GetGraph())
		{
			if (const TOptional<FVector2f> AnchorPosition = Graph->ResolveAnchor(Anchor))
			{
				return AnchorPosition.GetValue() + RelativePosition;
			}
		}
	}
	return Position;
}

void FScribbleNode::SetPosition(const FVector2f& InPosition)
{
	const FVector2f CurrentPosition = GetPosition(false);
	if (CurrentPosition.Equals(InPosition, 0.001f))
	{
		return;
	}

	Modify();
	Position = InPosition;
	UpdateRelativePosition(InPosition);
	PositionChangedEvent.Broadcast(Position);
	NotifyGraphChanged();
}

void FScribbleNode::UpdateRelativePosition(const FVector2f& InAbsolutePosition)
{
	RelativePosition = InAbsolutePosition;
	if (!Anchor.IsNone())
	{
		if (const FScribbleGraphData* Graph = GetGraph())
		{
			if (const TOptional<FVector2f> AnchorPosition = Graph->ResolveAnchor(Anchor))
			{
				RelativePosition = InAbsolutePosition - AnchorPosition.GetValue();
			}
		}
	}
}

void FScribbleNode::SetSize(const FVector2f& InSize)
{
	const FVector2f NewSize(FMath::Max<float>(1,InSize.X), FMath::Max<float>(1,InSize.Y));
	if (Size.Equals(NewSize, 0.001f))
	{
		return;
	}

	Modify();
	Size = NewSize;
	SizeChangedEvent.Broadcast(Size);
	NotifyGraphChanged();
}

bool FScribbleNode::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FScribbleObjectVersion::GUID);

	if (Ar.IsSaving())
	{
		// update the position using the potential anchor
		Position = GetPosition(true);
	}
	
	Ar << Guid;
	Ar << Anchor;
	Ar << Position;
	Ar << RelativePosition;
	Ar << Size;
	return true;
}

bool FScribbleNode::Identical(const FScribbleNode* Other, uint32 PortFlags) const
{
	return
		Type == Other->Type &&
		Guid == Other->Guid &&
		Anchor == Other->Anchor &&
		(Position - Other->Position).IsNearlyZero() &&
		(RelativePosition - Other->RelativePosition).IsNearlyZero() &&
		(Size - Other->Size).IsNearlyZero();
}

#if WITH_EDITOR

int32 FScribbleNode::OnPaint(const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId,
	const FWidgetStyle& InWidgetStyle, bool bSelected, float InZoomAmount) const
{
	return InLayerId;
}

#endif

const FScribbleGraphData* FScribbleNode::GetGraph() const
{
	if (WeakGraph.IsValid())
	{
		return WeakGraph.Pin().Get();
	}
	return nullptr;
}

FScribbleGraphData* FScribbleNode::GetGraph()
{
	if (WeakGraph.IsValid())
	{
		return WeakGraph.Pin().Get();
	}
	return nullptr;
}

void FScribbleNode::Modify()
{
	if (FScribbleGraphData* Graph = GetGraph())
	{
		Graph->Modify();
	}
}

void FScribbleNode::IncrementChangeBracket()
{
	ChangeBracket++;
}

void FScribbleNode::DecrementChangeBracket()
{
	ChangeBracket = FMath::Max(ChangeBracket - 1, 0);
	NotifyGraphChanged();
}

void FScribbleNode::NotifyGraphChanged()
{
	if (ChangeBracket == 0 && AccumulatedChanges > 0)
	{
		if (FScribbleGraphData* Graph = GetGraph())
		{
			Graph->NotifyChanged();
		}
		AccumulatedChanges = 0;
	}
	else
	{
		AccumulatedChanges++;
	}
}

bool FScribbleLineStrip::Serialize(FArchive& Ar)
{
	Ar << Vertices;
	Ar << Thickness;
	Ar << Color;
	return true;
}

bool FScribbleLineStrip::Identical(const FScribbleLineStrip* Other, uint32 PortFlags) const
{
	if (!Other)
	{
		return false;
	}

	if (Vertices.Num() != Other->Vertices.Num())
	{
		return false;
	}
	if (!FMath::IsNearlyEqual(Thickness, Other->Thickness))
	{
		return false;
	}
	if (!Color.Equals(Other->Color))
	{
		return false;
	}
	for(int32 VertexIndex = 0; VertexIndex < Vertices.Num(); VertexIndex++)
	{
		if (!Vertices[VertexIndex].Equals(Other->Vertices[VertexIndex]))
		{
			return false;
		}
	}
	
	return true;
}

bool FLineStripScribbleNode::Serialize(FArchive& Ar)
{
	if (!FScribbleNode::Serialize(Ar))
	{
		return false;
	}
	Ar << LineStrips;
	return true;
}

bool FLineStripScribbleNode::Identical(const FScribbleNode* Other, uint32 PortFlags) const
{
	if (!FScribbleNode::Identical(Other, PortFlags))
	{
		return false;
	}

	const FLineStripScribbleNode* OtherNode = CastChecked<FLineStripScribbleNode>(Other);
	if(LineStrips.Num() != OtherNode->LineStrips.Num())
	{
		return false;
	}

	for (int32 LineStripIndex = 0; LineStripIndex < LineStrips.Num(); LineStripIndex++)
	{
		const FScribbleLineStrip& LineStrip = LineStrips[LineStripIndex];
		const FScribbleLineStrip& OtherLineStrip = OtherNode->LineStrips[LineStripIndex];
		if (!LineStrip.Identical(&OtherLineStrip, PortFlags))
		{
			return false;
		}
	}
	
	return true;
}

void FLineStripScribbleNode::OffsetPosition(const FVector2f& InNewPosition)
{
	const FVector2f CurrentPosition = GetPosition();
	if (InNewPosition.Equals(CurrentPosition, 0.001f))
	{
		return;
	}

	Modify();
	IncrementChangeBracket();

	FVector2f NewSize = FVector2f(1,1);
	const FVector2f Delta = CurrentPosition - InNewPosition;
	for (FScribbleLineStrip& LineStrip : LineStrips)
	{
		for (FVector2f& Vertex : LineStrip.Vertices)
		{
			Vertex += Delta;
			NewSize.X = FMath::Max<float>(NewSize.X, Vertex.X);
			NewSize.Y = FMath::Max<float>(NewSize.Y, Vertex.Y);
		}
	}

	SetPosition(InNewPosition);
	SetSize(NewSize);

	DecrementChangeBracket();
}

bool FLineStripScribbleNode::DownSample(float InPrecision)
{
	Modify();
	IncrementChangeBracket();

	TArray<FScribbleLineStrip> NewLineStrips;
	for (FScribbleLineStrip& LineStrip : LineStrips)
	{
		FScribbleLineStrip NewLineStrip;
		NewLineStrip.Color = LineStrip.Color;
		NewLineStrip.Thickness = LineStrip.Thickness;

		for (int32 Index = 0; Index < LineStrip.Vertices.Num(); Index++)
		{
			const FVector2f& Vertex = LineStrip.Vertices[Index];
			if (NewLineStrip.Vertices.IsEmpty())
			{
				NewLineStrip.Vertices.Add(Vertex);
				continue;
			}

			// filter out points too close to each other
			const FVector2f& LastVertex = NewLineStrip.Vertices.Last();
			if (FVector2f::Distance(Vertex, LastVertex) < InPrecision)
			{
				NotifyGraphChanged();
				continue;
			}

			if (NewLineStrip.Vertices.Num() > 1)
			{
				// if the last vertex is between the new one and the one before
				// they may form a straight line
				const FVector2f& LastLastVertex = NewLineStrip.Vertices[NewLineStrip.Vertices.Num() - 2];
				const FVector2f& PointOnSegment = FMath::ClosestPointOnSegment2D(LastVertex, LastLastVertex, Vertex);
				if (FVector2f::Distance(LastVertex, PointOnSegment) < InPrecision)
				{
					// project the new vertex towards the last vertex's direction
					const FVector2f LastVertexWithSameDistance = LastLastVertex + (LastVertex - LastLastVertex).GetSafeNormal() * FVector2f::Distance(Vertex, LastLastVertex);

					// replace the last vertex with this one
					NotifyGraphChanged();
					NewLineStrip.Vertices.Last() = (LastVertexWithSameDistance + Vertex) * 0.5f;
					continue;
				}
			}

			NewLineStrip.Vertices.Add(Vertex);
		}

		// filter out empty line strips
		if (NewLineStrip.Vertices.Num() <= 1)
		{
			NotifyGraphChanged();
			continue;
		}

		// filter out line strips that are too small
		FBox2f Bounds = FBox2f(EForceInit::ForceInit);
		for (const FVector2f& Vertex : NewLineStrip.Vertices)
		{
			Bounds += Vertex;
		}
		if (Bounds.GetSize().X < InPrecision && Bounds.GetSize().Y < InPrecision)
		{
			NotifyGraphChanged();
			continue;
		}
		
		NewLineStrips.Add(NewLineStrip);
	}

	Swap(LineStrips, NewLineStrips);
	DecrementChangeBracket();

	return !LineStrips.IsEmpty();
}

FBox2f FLineStripScribbleNode::GetContentBounds() const
{
	FBox2f Bounds(EForceInit::ForceInit);
	for (const FScribbleLineStrip& LineStrip : LineStrips)
	{
		for (const FVector2f& Vertex : LineStrip.Vertices)
		{
			Bounds += Vertex;
		}
	}
	return Bounds;
}

#if WITH_EDITOR

int32 FLineStripScribbleNode::OnPaint(const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId,
                                      const FWidgetStyle& InWidgetStyle, bool bSelected, float InZoomAmount) const
{
	int32 LastLayerId = FScribbleNode::OnPaint(InAllottedGeometry, OutDrawElements, InLayerId, InWidgetStyle, bSelected, InZoomAmount);
	
	const FPaintGeometry PaintGeometry = InAllottedGeometry.ToPaintGeometry();
	
	TOptional<FSlateResourceHandle> WhiteBrushResourceHandle;
	for (int32 LineStripIndex = 0; LineStripIndex < LineStrips.Num(); LineStripIndex++)
	{
		const FScribbleLineStrip& LineStrip = LineStrips[LineStripIndex];
		if (LineStrip.Vertices.Num() < 2)
		{
			continue;
		}
		FLinearColor Color = LineStrip.Color;
		if (bSelected)
		{
			Color = UScribbleEditorSettings::Get()->SelectionColor;
		}

		const float Thickness = LineStrip.Thickness * InZoomAmount;
		if (Thickness < 0.75f)
		{
			continue;
		}
		
		FSlateDrawElement::MakeLines(OutDrawElements, ++LastLayerId, PaintGeometry, LineStrip.Vertices, ESlateDrawEffect::None, Color, false, Thickness);

		if (Thickness > 4.f && LineStrip.Vertices.Num() > 2)
		{
			VertexBuffer.Reset();
			IndexBuffer.Reset();
			VertexBuffer.Reserve((LineStrip.Vertices.Num() - 1) * 3);
			IndexBuffer.Reserve((LineStrip.Vertices.Num() - 1) * 3);

			const FSlateRenderTransform& Transform = PaintGeometry.GetAccumulatedRenderTransform();

			const float HalfThickness = LineStrip.Thickness * 0.5f;
			const FColor CustomVertColor = Color.ToFColor(true);
			const FVector2f* VertexA = &LineStrip.Vertices[0];
			const FVector2f* VertexB = VertexA + 1;
			const FVector2f* VertexC = VertexB + 1;
			for (int32 Index = 2; Index < LineStrip.Vertices.Num(); Index++)
			{
				// todo: this likely has to be optimized. we should potentially also cache
				// of these results for a known last paint geometry
				const FVector2f DeltaABUnit = ((*VertexA) - (*VertexB)).GetSafeNormal();
				const FVector2f NormalAB(-DeltaABUnit.Y, DeltaABUnit.X);
				const FVector2f DeltaCBUnit = ((*VertexC) - (*VertexB)).GetSafeNormal();
				const FVector2f NormalCB(-DeltaCBUnit.Y, DeltaCBUnit.X);
				
				const float Dot = FMath::Abs<float>(DeltaABUnit.Dot(DeltaCBUnit));
				if (Dot < 1.f)
				{
					float FlipA = 1.f;
					if (DeltaCBUnit.Dot(NormalAB) > 0)
					{
						FlipA = -1.f;
					}
					float FlipB = 1.f;
					if (DeltaABUnit.Dot(NormalCB) > 0)
					{
						FlipB = -1.f;
					}

					FVector2f CornerB1 = (*VertexB) + NormalAB * HalfThickness * FlipA;
					FVector2f CornerB2 = (*VertexB) + NormalCB * HalfThickness * FlipB;
					FVector2f CornerB3 = (*VertexB) - NormalAB * HalfThickness * FlipA;
					FVector2f CornerB4 = (*VertexB) - NormalCB * HalfThickness * FlipB;

					IndexBuffer.Add(VertexBuffer.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(
						Transform,
						CornerB1,
						{0.f,0.f},
						CustomVertColor
					)));
					IndexBuffer.Add(VertexBuffer.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(
						Transform,
						CornerB2,
						{0.f,0.f},
						CustomVertColor
					)));
					IndexBuffer.Add(VertexBuffer.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(
						Transform,
						FMath::Lerp(CornerB3, CornerB4, 0.5f),
						{0.f,0.f},
						CustomVertColor
					)));
				}
				
				VertexA = VertexB;
				VertexB = VertexC;
				VertexC++;
			}

			if (!VertexBuffer.IsEmpty() && !IndexBuffer.IsEmpty())
			{
				if (!WhiteBrushResourceHandle.IsSet())
				{
					WhiteBrushResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(*FAppStyle::GetBrush("WhiteBrush"));
				}
				FSlateDrawElement::MakeCustomVerts(OutDrawElements, LastLayerId, WhiteBrushResourceHandle.GetValue(), VertexBuffer, IndexBuffer, nullptr, 0, 0);
			}
		}
	}
	return LastLayerId;
}

bool FLineStripScribbleNode::AppendMouseEvent(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, const FVector2f& InPanelPosition)
{
	if (LineStrips.IsEmpty())
	{
		return false;
	}

	const FVector2f PositionInNode = InPanelPosition - GetPosition();

	FScribbleLineStrip& LineStrip = LineStrips.Last();
	if (!LineStrip.Vertices.IsEmpty())
	{
		if (LineStrip.Vertices.Last().Equals(PositionInNode, 1.f))
		{
			return false;
		}
	}

	LineStrip.Vertices.Add(PositionInNode);
	SetSize({
		FMath::Max<float>(Size.X, PositionInNode.X),
		FMath::Max<float>(Size.Y, PositionInNode.Y)
	});
	return true;
}

bool FLineStripScribbleNode::IntersectsCursorPosition(const FVector2f& InLocalPosition) const
{
	for (const FScribbleLineStrip& LineStrip : LineStrips)
	{
		for (int32 Index = 1; Index < LineStrip.Vertices.Num(); Index++)
		{
			const FVector2f& VertexA = LineStrip.Vertices[Index-1];
			const FVector2f& VertexB = LineStrip.Vertices[Index];
			const FVector2f& PointOnSegment = FMath::ClosestPointOnSegment2D(InLocalPosition, VertexA, VertexB);
			const float Distance = FVector2f::Distance(InLocalPosition, PointOnSegment);
			if (Distance < LineStrip.Thickness + 8.f)
			{
				return true;
			}
		}
	}
	return false;
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
