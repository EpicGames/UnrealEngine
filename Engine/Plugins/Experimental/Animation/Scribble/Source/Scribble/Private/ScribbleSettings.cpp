// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScribbleSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScribbleSettings)

UScribbleEditorSettings::UScribbleEditorSettings(const FObjectInitializer& ObjectInitializer)
    : Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
    , Color(FLinearColor::Blue)
	, SelectionColor(FColor::FromHex(TEXT("515151FF")))
	, AnchorColor(FColor::FromHex(TEXT("515151FF")))
    , Thickness(4.f)
    , Precision(3.f)
#endif
{
}

#if WITH_EDITOR

float UScribbleEditorSettings::GetSmoothing() const
{
	return FMath::Clamp((Precision - 2.f) / 12.f, 0.0f, 1.0f);
}

void UScribbleEditorSettings::SetSmoothing(float InValue)
{
	Precision = 2.f + InValue * 12.f;
}

#endif
