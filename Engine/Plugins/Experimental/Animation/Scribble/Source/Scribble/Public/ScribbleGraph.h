// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScribbleNode.h"
#include "HAL/IConsoleManager.h"
#include "ScribbleGraph.generated.h"

#define UE_API SCRIBBLE_API

extern UE_API TAutoConsoleVariable<bool> CVarScribbleEnabled;

DECLARE_DELEGATE_RetVal(FName, FScribbleGetCurrentAnchor)
DECLARE_DELEGATE_RetVal_OneParam(TOptional<FVector2f>, FScribbleResolveAnchor, const FName&)

/**
 * The backing data for a scribble graph. Given we want this to be used as a TSharedPtr 
 * it's separated from the UStruct graph.
 * This is used for the data backend only and corresponds with
 * the UScribbleEdGraphNode as well as the SScribbleGraphNode.
 */
struct FScribbleGraphData : public TSharedFromThis<FScribbleGraphData>
{
public:

	FScribbleGraphData()
		: ViewOffset(FVector2f::ZeroVector)
		, ZoomAmount(1.f)
		, ChangeBracket(0)
		, AccumulatedChanges(0)
	{
	}

	UE_API void Reset();
	UE_API bool IsEmpty() const;
	UE_API int32 NumNodes() const;
	UE_API FScribbleNode* GetNode(int32 InNodeIndex);
	UE_API const FScribbleNode* GetNode(int32 InNodeIndex) const;
	UE_API TSharedPtr<FScribbleNode> GetNodePtr(int32 InNodeIndex) const;
	UE_API FScribbleNode* FindNode(const FGuid& InId);
	UE_API const FScribbleNode* FindNode(const FGuid& InId) const;
	UE_API TSharedPtr<FScribbleNode> FindNodePtr(const FGuid& InId) const;
	UE_API FScribbleNode* operator[](int32 InNodeIndex);
	UE_API const FScribbleNode* operator[](int32 InNodeIndex) const;
	TArray<TSharedPtr<FScribbleNode>>::RangedForIteratorType begin() { return Nodes.begin(); }
	TArray<TSharedPtr<FScribbleNode>>::RangedForIteratorType end() { return Nodes.end(); }
	TArray<TSharedPtr<FScribbleNode>>::RangedForConstIteratorType begin() const { return Nodes.begin(); }
	TArray<TSharedPtr<FScribbleNode>>::RangedForConstIteratorType end() const { return Nodes.end(); }

	UE_API FGuid AddNode(const TSharedPtr<FScribbleNode>& InNode);
	UE_API bool RemoveNode(const TSharedPtr<FScribbleNode>& InNode);
	UE_API bool RemoveNode(const FGuid& InId);

	UE_API bool SupportsAnchors() const;
	UE_API FName GetCurrentAnchor() const;
	UE_API TOptional<FVector2f> ResolveAnchor(const FName& InName) const;

	UE_API TSharedPtr<FScribbleNode> GroupNodes(const TArray<TSharedPtr<FScribbleNode>>& InNodes);
	UE_API TArray<TSharedPtr<FScribbleNode>> UngroupNode(const TSharedPtr<FScribbleNode>& InNode);

	FSimpleMulticastDelegate& OnModify() { return OnModifyEvent; }
	FSimpleMulticastDelegate& OnChanged() { return OnChangedEvent; }
	FScribbleGetCurrentAnchor& OnGetCurrentAnchor() { return GetCurrentAnchorDelegate; }
	FScribbleResolveAnchor& OnResolveAnchor() { return ResolveAnchorDelegate; }

	UE_API bool Serialize(FArchive& Ar);
	UE_API void Modify();

	const FVector2f& GetViewOffset() const { return ViewOffset; }
	float GetZoomAmount() const { return ZoomAmount; }
	UE_API void SetView(const FVector2f& InViewOffset, float InZoomAmount);
	
protected:

	UE_API void IncrementChangeBracket();
	UE_API void DecrementChangeBracket();
	UE_API void NotifyChanged();

	FVector2f ViewOffset;
	float ZoomAmount;

	TArray<TSharedPtr<FScribbleNode>> Nodes;
	TMap<FGuid, TSharedPtr<FScribbleNode>> GuidToNode;
	FSimpleMulticastDelegate OnModifyEvent;
	FSimpleMulticastDelegate OnChangedEvent;
	FScribbleGetCurrentAnchor GetCurrentAnchorDelegate;
	FScribbleResolveAnchor ResolveAnchorDelegate;

	int32 ChangeBracket;
	int32 AccumulatedChanges;

	friend struct FScribbleNode;
	friend class UScribbleEdGraph;
};

/**
 * The Scribbe graph struct used mainly for serialization and as a data type for UPROPERTY macros.
 */
USTRUCT()
struct FScribbleGraph
{
	GENERATED_BODY()

public:

	FScribbleGraph()
	{
	}

	UE_API void Reset();
	UE_API bool IsValid() const;
	UE_API FScribbleGraphData* GetData();
	UE_API const FScribbleGraphData* GetData() const;
	TSharedPtr<FScribbleGraphData> GetDataPtr() const { return Data; }
	UE_API TSharedPtr<FScribbleGraphData> GetOrCreateDataPtr();

	UE_API bool Serialize(FArchive& Ar);
	bool Identical(const FScribbleGraph* Other, uint32 PortFlags) const;

	friend FArchive& operator<<(FArchive& Ar, FScribbleGraph& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

private:

	TSharedPtr<FScribbleGraphData> Data;
};

template<>
struct TStructOpsTypeTraits<FScribbleGraph> : public TStructOpsTypeTraitsBase2<FScribbleGraph>
{
	enum 
	{
		WithSerializer = true, // struct has a Serialize function for serializing its state to an FArchive.
		WithIdentical = true, // struct can be compared via an Identical(const T* Other, uint32 PortFlags) function.
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

#undef UE_API
