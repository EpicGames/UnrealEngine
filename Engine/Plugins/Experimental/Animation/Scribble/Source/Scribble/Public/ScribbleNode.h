// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITOR
#include "Input/Events.h"
#include "Layout/PaintGeometry.h"
#include "Rendering/DrawElements.h"
#include "Styling/WidgetStyle.h"
#endif

#include "ScribbleNode.generated.h"

struct FScribbleGraphData;

#define UE_API SCRIBBLE_API

DECLARE_MULTICAST_DELEGATE_OneParam(FScribbleNodeDimensionsChanged, const FVector2f&);

#define DECLARE_SCRIBBLE_NODE_METHODS(ScribbleNodeType) \
virtual UScriptStruct* GetScriptStruct() const override { return ScribbleNodeType::StaticStruct(); } \
template<typename T> \
friend const T* Cast(const ScribbleNodeType* InNode) \
{ \
return Cast<T>((const FScribbleNode*) InNode); \
} \
template<typename T> \
friend T* Cast(ScribbleNodeType* InNode) \
{ \
return Cast<T>((FScribbleNode*) InNode); \
} \
template<typename T> \
friend const T* CastChecked(const ScribbleNodeType* InNode) \
{ \
return CastChecked<T>((const FScribbleNode*) InNode); \
} \
template<typename T> \
friend T* CastChecked(ScribbleNodeType* InNode) \
{ \
return CastChecked<T>((FScribbleNode*) InNode); \
} \
static bool IsClassOf(const FScribbleNode* InNode) \
{ \
return InNode->GetScriptStruct()->IsChildOf(StaticStruct()); \
}

UENUM()
namespace EScribbleNodeType
{
	enum Type : uint8
	{
		Invalid,
		LineStrip,
	};
}

/*
 * A node in a scribble graph. This is used for the data backend only and corresponds with
 * the UScribbleEdGraphNode as well as the SScribbleGraphNode.
 */
USTRUCT()
struct FScribbleNode
{
	GENERATED_BODY()

public:

	FScribbleNode()
		: Type(EScribbleNodeType::Invalid)
		, Guid(FGuid())
		, Anchor(NAME_None)
		, Position(FVector2f::ZeroVector)
		, RelativePosition(FVector2f::ZeroVector)
		, Size(1,1)
		, ChangeBracket(0)
		, AccumulatedChanges(0)
	{
	}

	FScribbleNode(const EScribbleNodeType::Type& InType)
		: Type(InType)
		, Guid(FGuid::NewGuid())
		, Anchor(NAME_None)
		, Position(FVector2f::ZeroVector)
		, RelativePosition(FVector2f::ZeroVector)
		, Size(1,1)
		, ChangeBracket(0)
		, AccumulatedChanges(0)
	{
	}

	virtual ~FScribbleNode()
	{
	}

	const EScribbleNodeType::Type GetType() const { return Type; }
	const FGuid& GetId() const { return Guid; }
	const FName& GetAnchor() const { return Anchor; }
	UE_API void SetAnchor(const FName& InAnchor);

	UE_API FVector2f GetPosition(bool bResolveAnchor = true) const;
	const FVector2f& GetRelativePosition() const { return RelativePosition; }
	UE_API void SetPosition(const FVector2f& InPosition);
	virtual void OffsetPosition(const FVector2f& InNewPosition) {}
	virtual FBox2f GetContentBounds() const { return FBox2f(EForceInit::ForceInitToZero); }
	const FVector2f& GetSize() const { return Size; }
	UE_API void SetSize(const FVector2f& InSize);
	virtual bool DownSample(float InPrecision) { return false; }
	
	UE_API virtual bool Serialize(FArchive& Ar);
	UE_API virtual bool Identical(const FScribbleNode* Other, uint32 PortFlags) const;

	FScribbleNodeDimensionsChanged& OnPositionChanged() { return PositionChangedEvent; }
	FScribbleNodeDimensionsChanged& OnSizeChanged() { return SizeChangedEvent; }

#if WITH_EDITOR
	UE_API virtual int32 OnPaint(const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bSelected, float InZoomAmount) const;
	virtual bool AppendMouseEvent(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, const FVector2f& InPanelPosition) { return false; }
	virtual bool IntersectsCursorPosition(const FVector2f& InLocalPosition) const { return false; }
#endif

	static bool IsClassOf(const FScribbleNode* InNode)
	{
		return true;
	}

	virtual UScriptStruct* GetScriptStruct() const { return FScribbleNode::StaticStruct(); }

	bool IsA(const UScriptStruct* InScriptStruct) const
	{
		return GetScriptStruct()->IsChildOf(InScriptStruct);
	}

	template<typename T>
	bool IsA() const { return T::IsClassOf(this); }

	template<typename T>
	friend const T* Cast(const FScribbleNode* InNode)
	{
		if(InNode)
		{
			if(InNode->IsA<T>())
			{
				return static_cast<const T*>(InNode);
			}
		}
		return nullptr;
	}

	template<typename T>
	friend T* Cast(FScribbleNode* InNode)
	{
		if(InNode)
		{
			if(InNode->IsA<T>())
			{
				return static_cast<T*>(InNode);
			}
		}
		return nullptr;
	}

	template<typename T>
	friend const T* CastChecked(const FScribbleNode* InNode)
	{
		const T* Node = Cast<T>(InNode);
		check(Node);
		return Node;
	}

	template<typename T>
	friend T* CastChecked(FScribbleNode* InNode)
	{
		T* Node = Cast<T>(InNode);
		check(Node);
		return Node;
	}

	UE_API const FScribbleGraphData* GetGraph() const;
	UE_API FScribbleGraphData* GetGraph();

protected:

	UE_API void Modify();
	UE_API void IncrementChangeBracket();
	UE_API void DecrementChangeBracket();
	UE_API void NotifyGraphChanged();

	void UpdateRelativePosition(const FVector2f& InAbsolutePosition);

	UPROPERTY()
	TEnumAsByte<EScribbleNodeType::Type> Type; 

	UPROPERTY()
	FGuid Guid;

	UPROPERTY()
	FName Anchor;

	UPROPERTY()
	FVector2f Position;

	UPROPERTY()
	FVector2f RelativePosition;

	UPROPERTY()
	FVector2f Size;

	TWeakPtr<FScribbleGraphData> WeakGraph;
	FScribbleNodeDimensionsChanged PositionChangedEvent;
	FScribbleNodeDimensionsChanged SizeChangedEvent;

	int32 ChangeBracket;
	int32 AccumulatedChanges;

	friend struct FScribbleGraphData;
	friend class SScribbleGraphPanel;
};

USTRUCT()
struct FScribbleLineStrip
{
	GENERATED_BODY()

public:

	FScribbleLineStrip()
		: Thickness(1.f)
		, Color(FLinearColor::White)
	{
	}

	UPROPERTY(EditAnywhere, Category=Scribble)
	TArray<FVector2f> Vertices;

	UPROPERTY(EditAnywhere, Category=Scribble)
	float Thickness;

	UPROPERTY(EditAnywhere, Category=Scribble)
	FLinearColor Color;

	UE_API bool Serialize(FArchive& Ar);
	UE_API bool Identical(const FScribbleLineStrip* Other, uint32 PortFlags) const;

	friend FArchive& operator<<(FArchive& Ar, FScribbleLineStrip& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};

USTRUCT()
struct FLineStripScribbleNode : public FScribbleNode
{
	GENERATED_BODY()
	DECLARE_SCRIBBLE_NODE_METHODS(FLineStripScribbleNode)

	FLineStripScribbleNode()
	: FScribbleNode(EScribbleNodeType::LineStrip)
	{
	}

	UPROPERTY(EditAnywhere, Category=Scribble)
	TArray<FScribbleLineStrip> LineStrips;

	UE_API virtual bool Serialize(FArchive& Ar) override;
	UE_API virtual bool Identical(const FScribbleNode* Other, uint32 PortFlags) const override;
	UE_API virtual void OffsetPosition(const FVector2f& InNewPosition) override;
	UE_API virtual bool DownSample(float InPrecision) override;
	UE_API virtual FBox2f GetContentBounds() const override;;

#if WITH_EDITOR
	UE_API virtual int32 OnPaint(const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bSelected, float InZoomAmount) const override;
	UE_API virtual bool AppendMouseEvent(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, const FVector2f& InPanelPosition) override;
	UE_API virtual bool IntersectsCursorPosition(const FVector2f& InLocalPosition) const override;
	
	mutable TArray<FSlateVertex> VertexBuffer;
	mutable TArray<SlateIndex> IndexBuffer;
#endif
};

#undef UE_API
