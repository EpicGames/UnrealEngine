// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

#define UE_API SCRIBBLE_API

// Custom serialization version for scribble plugin
struct FScribbleObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

private:
	FScribbleObjectVersion() {}
};

#undef UE_API
