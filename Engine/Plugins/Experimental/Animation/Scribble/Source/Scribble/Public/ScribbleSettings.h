// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScribbleSettings.h: Declares the ScribbleSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"

#include "ScribbleSettings.generated.h"

#define UE_API SCRIBBLE_API

/**
 * Default Scribble settings.
 */
UCLASS(config = EditorPerProjectUserSettings, meta=(DisplayName="Scribble"), MinimalAPI)
class UScribbleEditorSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:

#if WITH_EDITORONLY_DATA

	// The color used for shapes
	UPROPERTY(EditAnywhere, config, Category = Scribble)
	FLinearColor Color;

	// The color used to represent selection
	UPROPERTY(EditAnywhere, config, Category = Scribble)
	FLinearColor SelectionColor;

	// The color used to draw the anchor line
	UPROPERTY(EditAnywhere, config, Category = Scribble)
	FLinearColor AnchorColor;

	// The thickness used for lines
	UPROPERTY(EditAnywhere, config, Category = Scribble)
	float Thickness;

	// Defines how exact lines are - and how small the smallest possible node is going to be
	UPROPERTY(EditAnywhere, config, Category = Scribble)
	float Precision;

#endif

#if WITH_EDITOR

	UE_API float GetSmoothing() const;
	UE_API void SetSmoothing(float InValue);

	void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);

		if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			// Dragging spinboxes causes this to be called every frame so we wait until they've finished dragging before saving.
			SaveConfig();
		}
		
		OnSettingChanged().Broadcast(this, PropertyChangedEvent);
	}
#endif

	static UScribbleEditorSettings* Get() { return GetMutableDefault<UScribbleEditorSettings>(); }
};

#undef UE_API
