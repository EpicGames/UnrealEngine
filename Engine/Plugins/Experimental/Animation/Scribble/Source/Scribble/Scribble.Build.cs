// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class Scribble : ModuleRules
    {
        public Scribble(ReadOnlyTargetRules Target) : base(Target)
        {
			NumIncludedBytesPerUnityCPPOverride = 688128; // best unity size found from using UBT ProfileUnitySizes mode

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
                    "SlateCore",
                    "Slate",
                    "Engine",
				}
            );

            PublicDependencyModuleNames.AddRange(
                new string[]
                {
	                "DeveloperSettings",
                }
            );
        }
    }
}
