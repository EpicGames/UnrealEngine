// Copyright Epic Games, Inc. All Rights Reserved.

#include "SScribbleGraph.h"

#include "EdGraphSchema_K2_Actions.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Selection.h"
#include "Settings/EditorStyleSettings.h"
#include "ScribbleEdGraph.h"
#include "ScribbleEdGraphSchema.h"
#include "ScribbleEdGraphCommands.h"
#include "ScribbleSettings.h"
#include "SScribbleGraphNode.h"
#include "UnrealEdGlobals.h"
#include "Scribble/Public/ScribbleSettings.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SScribbleGraph"

struct FScribbleZoomLevelsContainer : public FZoomLevelsContainer
{
	struct FScribbleZoomLevelEntry
	{
	public:
		FScribbleZoomLevelEntry(float InZoomAmount, const FText& InDisplayText, EGraphRenderingLOD::Type InLOD)
			: DisplayText(FText::Format(NSLOCTEXT("GraphEditor", "Zoom", "Zoom {0}"), InDisplayText))
		, ZoomAmount(InZoomAmount)
		, LOD(InLOD)
		{
		}

	public:
		FText DisplayText;
		float ZoomAmount;
		EGraphRenderingLOD::Type LOD;
	};
	
	FScribbleZoomLevelsContainer()
	{
		ZoomLevels.Reserve(22);
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.025f, FText::FromString(TEXT("-14")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.070f, FText::FromString(TEXT("-13")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.100f, FText::FromString(TEXT("-12")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.125f, FText::FromString(TEXT("-11")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.150f, FText::FromString(TEXT("-10")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.175f, FText::FromString(TEXT("-9")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.200f, FText::FromString(TEXT("-8")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.225f, FText::FromString(TEXT("-7")), EGraphRenderingLOD::LowDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.250f, FText::FromString(TEXT("-6")), EGraphRenderingLOD::LowDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.375f, FText::FromString(TEXT("-5")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.500f, FText::FromString(TEXT("-4")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.675f, FText::FromString(TEXT("-3")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.750f, FText::FromString(TEXT("-2")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(0.875f, FText::FromString(TEXT("-1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(1.000f, FText::FromString(TEXT("1:1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(1.250f, FText::FromString(TEXT("+1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(1.375f, FText::FromString(TEXT("+2")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FScribbleZoomLevelEntry(1.500f, FText::FromString(TEXT("+3")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FScribbleZoomLevelEntry(1.675f, FText::FromString(TEXT("+4")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FScribbleZoomLevelEntry(1.750f, FText::FromString(TEXT("+5")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FScribbleZoomLevelEntry(1.875f, FText::FromString(TEXT("+6")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FScribbleZoomLevelEntry(2.000f, FText::FromString(TEXT("+7")), EGraphRenderingLOD::FullyZoomedIn));
	}

	float GetZoomAmount(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].ZoomAmount;
	}

	int32 GetNearestZoomLevel(float InZoomAmount) const override
	{
		for (int32 ZoomLevelIndex=0; ZoomLevelIndex < GetNumZoomLevels(); ++ZoomLevelIndex)
		{
			if (InZoomAmount <= GetZoomAmount(ZoomLevelIndex))
			{
				return ZoomLevelIndex;
			}
		}

		return GetDefaultZoomLevel();
	}
	
	FText GetZoomText(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].DisplayText;
	}
	
	int32 GetNumZoomLevels() const override
	{
		return ZoomLevels.Num();
	}
	
	int32 GetDefaultZoomLevel() const override
	{
		return 14;
	}

	EGraphRenderingLOD::Type GetLOD(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].LOD;
	}

	TArray<FScribbleZoomLevelEntry> ZoomLevels;
};


SScribbleGraphPanel::SScribbleGraphPanel()
: ScribbleGraphObj(nullptr)
, bAllowNavigation(true)
, RoundedFrameBrush(MakeUnique<FSlateRoundedBoxBrush>(FStyleColors::Transparent, 16.0f, FStyleColors::White, 2.0f))
{
}

SScribbleGraphPanel::~SScribbleGraphPanel()
{
	if (!GExitPurge)
	{
		if ( ensure(ScribbleGraphObj) )
		{
			ScribbleGraphObj->RemoveFromRoot();
		}
	}
}

/** Constructs this widget with InArgs */
void SScribbleGraphPanel::Construct(const FArguments& InArgs)
{
	// Create the graph
	ScribbleGraphObj = NewObject<UScribbleEdGraph>();
	ScribbleGraphObj->Schema = UScribbleEdGraphSchema::StaticClass();
	ScribbleGraphObj->AddToRoot();
	ScribbleGraphObj->Initialize(InArgs._GraphData);

	ScribbleEnabled = InArgs._ScribbleEnabled;
	ScribbleNodeType = InArgs._ScribbleNodeType;
	ScribbleColor = InArgs._ScribbleColor;
	if (!ScribbleColor.IsBound() && !ScribbleColor.IsSet())
	{
		ScribbleColor.BindLambda([]()
		{
			return UScribbleEditorSettings::Get()->Color;
		});
	}
	ScribbleThickness = InArgs._ScribbleThickness;
	if (!ScribbleThickness.IsBound() && !ScribbleThickness.IsSet())
	{
		ScribbleThickness.BindLambda([]()
		{
			return UScribbleEditorSettings::Get()->Thickness;
		});
	}
	ScribblePrecision = InArgs._ScribblePrecision;
	if (!ScribblePrecision.IsBound() && !ScribblePrecision.IsSet())
	{
		ScribblePrecision.BindLambda([]()
		{
			return UScribbleEditorSettings::Get()->Precision;
		});
	}
	bAllowNavigation = InArgs._AllowNavigation;
	WeakPanelToSync = InArgs._PanelToSync;

	if (WeakPanelToSync.IsValid())
	{
		if (TSharedPtr<SGraphPanel> PanelToSync = WeakPanelToSync.Pin())
		{
			ZoomLevel = PanelToSync->GetZoomLevel();
			SetZoomLevelsContainer(PanelToSync->GetZoomLevels());
		}
	}
	else
	{
		ZoomLevel = FScribbleZoomLevelsContainer().GetDefaultZoomLevel();
		SetZoomLevelsContainer<FScribbleZoomLevelsContainer>();
	}

	SGraphPanel::FArguments GraphPanelArgs;
	GraphPanelArgs.GraphObj(ScribbleGraphObj);
	GraphPanelArgs.OnSelectionChanged(this, &SScribbleGraphPanel::HandleSelectionChanged);
	GraphPanelArgs.ShouldDrawBackground(InArgs._ShouldDrawBackground);
	GraphPanelArgs.ShouldDrawSurroundingShadow(InArgs._ShouldDrawSurroundingShadow);
	GraphPanelArgs.AllowZoom(bAllowNavigation);
	GraphPanelArgs.AllowPanning(bAllowNavigation);
	
	SGraphPanel::Construct(GraphPanelArgs);

	BindCommands();

	Update();

	GEditor->RegisterForUndo(this);
}

bool SScribbleGraphPanel::IsScribbleEnabled() const
{
	return ScribbleEnabled.Get(false);
}

bool SScribbleGraphPanel::HasActiveNodeType() const
{
	return GetActiveNodeType() != EScribbleNodeType::Invalid;
}

void SScribbleGraphPanel::ActivateSelectionMode()
{
	SetActiveNodeType(EScribbleNodeType::Invalid);
}

EScribbleNodeType::Type SScribbleGraphPanel::GetActiveNodeType() const
{
	return ScribbleNodeType.Get(EScribbleNodeType::Invalid);
}

void SScribbleGraphPanel::SetActiveNodeType(EScribbleNodeType::Type InNodeType)
{
	if (OnSetNodeType.IsBound())
	{
		OnSetNodeType.Execute(InNodeType);
		return;
	}
	if (!ScribbleNodeType.IsBound())
	{
		ScribbleNodeType = InNodeType;
	}
}

UScribbleEdGraph* SScribbleGraphPanel::GetScribbleEdGraph()
{
	return ScribbleGraphObj;
}

const UScribbleEdGraph* SScribbleGraphPanel::GetScribbleEdGraph() const
{
	return ScribbleGraphObj;
}

FScribbleGraphData* SScribbleGraphPanel::GetScribbleGraph()
{
	if (UScribbleEdGraph* ScribbleEdGraph = GetScribbleEdGraph())
	{
		return ScribbleEdGraph->GetGraphData();
	}
	return nullptr;
}

const FScribbleGraphData* SScribbleGraphPanel::GetScribbleGraph() const
{
	if (const UScribbleEdGraph* ScribbleEdGraph = GetScribbleEdGraph())
	{
		return ScribbleEdGraph->GetGraphData();
	}
	return nullptr;
}

FReply SScribbleGraphPanel::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (!IsScribbleEnabled())
	{
		return FReply::Unhandled();
	}
	if (HasActiveNodeType())
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			bIsScribbleDragging = true;
			UpdateNewNode(MyGeometry, MouseEvent);
			return FReply::Handled();
		}
	}
	const FReply ReplyFromGraphPanel = SGraphPanel::OnMouseButtonDown(MyGeometry, MouseEvent);
	if (ReplyFromGraphPanel.IsEventHandled())
	{
		if (NodeUnderMousePtr.IsValid())
		{
			return ReplyFromGraphPanel;
		}
	}

	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildNodes(MyGeometry, ArrangedChildren);

	// use an alternative path to look up nodes to select by click.
	// the nodes may be overlapping so the initial event may not catch it.
	const FVector2f ArrangedSpacePosition = MouseEvent.GetScreenSpacePosition();
	const int32 NumChildren = ArrangedChildren.Num();
	for( int32 ChildIndex=NumChildren-1; ChildIndex >= 0; --ChildIndex )
	{
		const FArrangedWidget& Candidate = ArrangedChildren[ChildIndex];
		if (Candidate.Geometry.IsUnderLocation( ArrangedSpacePosition ))
		{
			const FVector2f MousePositionInNode = Candidate.Geometry.AbsoluteToLocal(ArrangedSpacePosition);
			const TSharedRef<SScribbleGraphNode>& ScribbleGraphNode = StaticCastSharedRef<SScribbleGraphNode, SWidget>(Candidate.Widget);
			if (ScribbleGraphNode->CanBeSelected(MousePositionInNode))
			{
				// Track the node that we're dragging; we will move it in OnMouseMove.
				this->OnBeginNodeInteraction(ScribbleGraphNode, MousePositionInNode);
				return FReply::Handled().CaptureMouse( SharedThis(this) );
			}
		}
	}

	return ReplyFromGraphPanel;
}

void SScribbleGraphPanel::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	if (!IsScribbleEnabled())
	{
		return;
	}
	if (HasActiveNodeType())
	{
		if (bIsScribbleDragging.Get(false))
		{
			CancelNewNode();
		}
	}
	SGraphPanel::OnMouseCaptureLost(CaptureLostEvent);
}

FReply SScribbleGraphPanel::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (!IsScribbleEnabled())
	{
		return FReply::Unhandled();
	}
	if (HasActiveNodeType())
	{
		if (bIsScribbleDragging.Get(false))
		{
			UpdateNewNode(MyGeometry, MouseEvent);
			CommitNewNode();
			return FReply::Handled();
		}
	}
	return SGraphPanel::OnMouseButtonUp(MyGeometry, MouseEvent);
}

FReply SScribbleGraphPanel::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (!IsScribbleEnabled())
	{
		return FReply::Unhandled();
	}
	if (HasActiveNodeType())
	{
		if (bIsScribbleDragging.Get(false))
		{
			UpdateNewNode(MyGeometry, MouseEvent);
			return FReply::Handled();
		}
	}
	return SGraphPanel::OnMouseMove(MyGeometry, MouseEvent);
}

void SScribbleGraphPanel::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	if (!IsScribbleEnabled())
	{
		return;
	}
	SGraphPanel::OnMouseLeave(MouseEvent);
}

FReply SScribbleGraphPanel::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (!IsScribbleEnabled())
	{
		return FReply::Unhandled();
	}
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return SGraphPanel::OnKeyDown(MyGeometry, InKeyEvent);
}

void SScribbleGraphPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SGraphPanel::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	UpdateScribbleVisibility();

	if (WeakPanelToSync.IsValid())
	{
		if (TSharedPtr<SGraphPanel> PanelToSync = WeakPanelToSync.Pin())
		{
			if (IsScribbleEnabled())
			{
				PanelToSync->SetViewOffset(GetViewOffset());
				PanelToSync->SetZoomLevel(ZoomLevel);
			}
			else
			{
				SetViewOffset(PanelToSync->GetViewOffset());
				SetZoomLevel(PanelToSync->GetZoomLevel());
			}
		}
	}

	if (bAllowNavigation && !WeakPanelToSync.IsValid())
	{
		if (FScribbleGraphData* ScribbleGraph = GetScribbleGraph())
		{
			ScribbleGraph->SetView(GetViewOffset(), GetZoomAmount());
		}
	}

	if (const FScribbleGraphData* Graph = GetScribbleGraph())
	{
		if (Graph->SupportsAnchors())
		{
			FArrangedChildren ArrangedChildren(EVisibility::Visible);
			ArrangeChildNodes(AllottedGeometry, ArrangedChildren);

			for(int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ChildIndex++)
			{
				const FArrangedWidget& ArrangedWidget = ArrangedChildren[ChildIndex];
				TSharedRef<SScribbleGraphNode> GraphNode = StaticCastSharedRef<SScribbleGraphNode, SWidget>(ArrangedWidget.Widget);
				const FScribbleNode* ScribbleNode = GraphNode->GetScribbleNode();
				if (!ScribbleNode)
				{
					continue;
				}
				if (ScribbleNode->GetAnchor().IsNone())
				{
					continue;
				}
				UScribbleEdGraphNode* EdGraphNode = GraphNode->GetScribbleEdGraphNode();
				if (!EdGraphNode)
				{
					continue;
				}
				const FVector2f NodePosition = ScribbleNode->GetPosition(true);
				EdGraphNode->NodePosX = NodePosition.X;
				EdGraphNode->NodePosY = NodePosition.Y;
			}
		}
	}
}

int32 SScribbleGraphPanel::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 LastLayerId = LayerId;

	if (ScribbleEnabled.Get(false))
	{
		// draw a dark overlay below our content to indicate that we are in scribble mode
		if (!GetShouldDrawBackground())
		{
			static const FSlateBrush* DefaultBackground = FAppStyle::GetBrush(TEXT("Graph.Panel.SolidBackground"));
			static constexpr FLinearColor DarkenColor = FLinearColor(1,1,1, 0.6f);
			FSlateDrawElement::MakeBox(OutDrawElements, ++LastLayerId, AllottedGeometry.ToPaintGeometry(), DefaultBackground, ESlateDrawEffect::None, DarkenColor);
		}
	}
	
	if (GetZoomAmount() > 0.3f)
	{
		if (const FScribbleGraphData* Graph = GetScribbleGraph())
		{
			if (Graph->SupportsAnchors())
			{
				// update the outline color
				const FLinearColor LineColor = UScribbleEditorSettings::Get()->AnchorColor;
				const FLinearColor FillColor = LineColor * FLinearColor(0.35f,0.35f,0.35f,0.35f);;
				RoundedFrameBrush->OutlineSettings.Color = LineColor;
				RoundedFrameBrush->TintColor = FillColor;
					
				FArrangedChildren ArrangedChildren(EVisibility::Visible);
				ArrangeChildNodes(AllottedGeometry, ArrangedChildren);
				
				for(int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ChildIndex++)
				{
					const FArrangedWidget& ArrangedWidget = ArrangedChildren[ChildIndex];
					TSharedRef<SScribbleGraphNode> GraphNode = StaticCastSharedRef<SScribbleGraphNode, SWidget>(ArrangedWidget.Widget);
					const FScribbleNode* ScribbleNode = GraphNode->GetScribbleNode();
					if (!ScribbleNode)
					{
						continue;
					}
					if (ScribbleNode->GetAnchor().IsNone())
					{
						continue;
					}

					TOptional<FVector2f> AnchorPosition = Graph->ResolveAnchor(ScribbleNode->GetAnchor());
					if (!AnchorPosition.IsSet())
					{
						continue;
					}

					const FVector2f FrameOffset = FVector2f(20,20);
					const FVector2f ScribbleNodePositionWithOffset = ScribbleNode->GetPosition(true) - FrameOffset;
					const FVector2f ScribbleNodeSizeWithOffset = ScribbleNode->GetSize() + FrameOffset * 2.f;
					FVector2f ScribbleNodeCorner = ScribbleNodePositionWithOffset;

					// draw a rounded box around the anchored content
					const FVector2f ScribbleNodePositionInView = (ScribbleNodePositionWithOffset - GetViewOffset()) * GetZoomAmount();
             		const FVector2f ScribbleNodeSizeInView = ScribbleNodeSizeWithOffset * GetZoomAmount();
                    const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry(ScribbleNodeSizeInView, FSlateLayoutTransform(ScribbleNodePositionInView));
                    FSlateDrawElement::MakeBox(OutDrawElements, ++LastLayerId, PaintGeometry, RoundedFrameBrush.Get(), ESlateDrawEffect::None, FillColor);

					// determine the shortest path to the box
					const TArray<FVector2f> ScribbleNodeCorners = {
						ScribbleNodeCorner,
						ScribbleNodeCorner + (ScribbleNodeSizeWithOffset) * FVector2f(1.f, 0.f),
						ScribbleNodeCorner + (ScribbleNodeSizeWithOffset) * FVector2f(1.f, 1.f),
						ScribbleNodeCorner + (ScribbleNodeSizeWithOffset) * FVector2f(0.f, 1.f),
					};
					const TArray<FVector2f> ScribbleCandidates = {
						FMath::Lerp<FVector2f>(ScribbleNodeCorners[0], ScribbleNodeCorners[1], 0.1f),
						FMath::Lerp<FVector2f>(ScribbleNodeCorners[0], ScribbleNodeCorners[1], 0.9f),
						FMath::Lerp<FVector2f>(ScribbleNodeCorners[1], ScribbleNodeCorners[2], 0.1f),
						FMath::Lerp<FVector2f>(ScribbleNodeCorners[1], ScribbleNodeCorners[2], 0.9f),
						FMath::Lerp<FVector2f>(ScribbleNodeCorners[2], ScribbleNodeCorners[3], 0.1f),
						FMath::Lerp<FVector2f>(ScribbleNodeCorners[2], ScribbleNodeCorners[3], 0.9f),
						FMath::Lerp<FVector2f>(ScribbleNodeCorners[3], ScribbleNodeCorners[0], 0.1f),
						FMath::Lerp<FVector2f>(ScribbleNodeCorners[3], ScribbleNodeCorners[0], 0.9f)
					};

					float ClosestDistance = FLT_MAX;
					for (const FVector2f& ScribbleCandidate : ScribbleCandidates)
					{
						const float Distance = FVector2f::DistSquared(ScribbleCandidate, AnchorPosition.GetValue());
						if (Distance < ClosestDistance)
						{
							ClosestDistance = Distance;
							ScribbleNodeCorner = ScribbleCandidate;
						}
					}

					// draw a connecting line to the box 
					const TArray<FVector2f> AnchorVertices = {
						(AnchorPosition.GetValue() - GetViewOffset()) * GetZoomAmount(),
						(ScribbleNodeCorner - GetViewOffset()) * GetZoomAmount()
					};
					FSlateDrawElement::MakeLines(OutDrawElements, LastLayerId, AllottedGeometry.ToPaintGeometry(), AnchorVertices, ESlateDrawEffect::None, LineColor, false, RoundedFrameBrush->OutlineSettings.Width);
				}
			}
		}
	}

	return SGraphPanel::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

void SScribbleGraphPanel::PostUndo(bool bSuccess)
{
	if (UScribbleEdGraph* ScribbleEdGraph = GetScribbleEdGraph())
	{
		ScribbleEdGraph->RebuildGraph();
	}
}

void SScribbleGraphPanel::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void SScribbleGraphPanel::BindCommands()
{
	// create new command
	const FScribbleEdGraphCommands& Commands = FScribbleEdGraphCommands::Get();

	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(Commands.FrameSelection,
		FExecuteAction::CreateLambda([this] ()
		{
			ZoomToFit(!ScribbleGraphObj->GetSelectedNodeIds().IsEmpty());
		}),
		FCanExecuteAction::CreateLambda([this] () -> bool
		{
			return ScribbleGraphObj->GetNodes().Num() > 0;
		})		
	);

	CommandList->MapAction(Commands.DeleteSelection,
		FExecuteAction::CreateLambda([this] ()
		{
			const FScopedTransaction _(LOCTEXT("RemoveSelectedScribbleNodes", "Remove Selected Scribble Nodes"));
			ScribbleGraphObj->RemoveSelectedNodes();
		}),
		FCanExecuteAction::CreateLambda([this] () -> bool
		{
			return ScribbleGraphObj->GetSelectedNodeIds().Num() > 0;
		})		
	);

	CommandList->MapAction(Commands.SelectAll,
		FExecuteAction::CreateLambda([this] ()
		{
			ScribbleGraphObj->SelectAllNodes();
		}),
		FCanExecuteAction::CreateLambda([this] () -> bool
		{
			return true;
		})		
	);

	CommandList->MapAction(Commands.GroupSelection,
		FExecuteAction::CreateLambda([this] ()
		{
			const FScopedTransaction _(LOCTEXT("GroupSelectedScribbleNodes", "Group Scribble Nodes"));
			ScribbleGraphObj->GroupSelectedNodes();
		}),
		FCanExecuteAction::CreateLambda([this] () -> bool
		{
			return ScribbleGraphObj->GetSelectedNodeIds().Num() > 1;
		})		
	);

	CommandList->MapAction(Commands.UngroupSelection,
		FExecuteAction::CreateLambda([this] ()
		{
			const FScopedTransaction _(LOCTEXT("UngroupSelectedScribbleNode", "Ungroup Scribble Node"));
			ScribbleGraphObj->UngroupSelectedNodes();
		}),
		FCanExecuteAction::CreateLambda([this] () -> bool
		{
			return ScribbleGraphObj->GetSelectedNodeIds().Num() == 1;
		})		
	);

	CommandList->MapAction(Commands.SelectionTool,
		FExecuteAction::CreateLambda([this] ()
		{
			ActivateSelectionMode();
		})
	);

	CommandList->MapAction(Commands.BrushTool,
		FExecuteAction::CreateLambda([this] ()
		{
			SetActiveNodeType(EScribbleNodeType::LineStrip);
		})
	);
}

void SScribbleGraphPanel::UpdateScribbleVisibility()
{
	if (GetVisibility() == EVisibility::Visible ||
		GetVisibility() == EVisibility::HitTestInvisible)
	{
		SetVisibility(IsScribbleEnabled() ? EVisibility::Visible : EVisibility::HitTestInvisible);
	}
}

void SScribbleGraphPanel::HandleSelectionChanged(const FGraphPanelSelectionSet& SelectionSet)
{
	UScribbleEdGraph* ScribbleEdGraph = GetScribbleEdGraph();
	if (!ScribbleEdGraph)
	{
		return;
	}

	if (ScribbleEdGraph->bIsSelecting)
	{
		return;
	}
	const TGuardValue<bool> _(ScribbleEdGraph->bIsSelecting, true);

	TArray<FGuid> NodeIds;
	for (const UObject* SelectedObject : SelectionSet)
	{
		if (const UScribbleEdGraphNode* Node = Cast<UScribbleEdGraphNode>(SelectedObject))
		{
			NodeIds.Add(Node->GetNodeId());
		}
	}
	ScribbleEdGraph->SelectNodes(NodeIds);
}

bool SScribbleGraphPanel::UpdateNewNode(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!bIsScribbleDragging.Get(false))
	{
		return false;
	}
	
	if (!NewNode)
	{
		Transaction = MakeShared<FScopedTransaction>(LOCTEXT("AddScribbleNode", "Add Scribble Node"));
		if (FScribbleGraphData* ScribbleGraph = GetScribbleGraph())
		{
			ScribbleGraph->Modify();
		}
		switch (ScribbleNodeType.Get(EScribbleNodeType::LineStrip))
		{
			case EScribbleNodeType::LineStrip:
			{
				TSharedPtr<FLineStripScribbleNode> LineStripScribbleNode = MakeShared<FLineStripScribbleNode>();
				LineStripScribbleNode->LineStrips.Emplace();
				LineStripScribbleNode->LineStrips.Last().Thickness = ScribbleThickness.Get(4.f);
				LineStripScribbleNode->LineStrips.Last().Color = ScribbleColor.Get(FLinearColor::Blue);
				NewNode = LineStripScribbleNode;
				break;
			}
			default:
			{
				return false;
			}
		}

		NewNode->SetPosition(GetViewOffset());
		ScribbleGraphObj->AddScribbleNode(NewNode);
		ScribbleGraphObj->ClearSelection();
	}


	const FVector2f MousePosition = MouseEvent.GetScreenSpacePosition();
	const FVector2f GraphEditorLocation = GetViewOffset();
	const float GraphEditorZoomAmount = GetZoomAmount();
	const FVector2f WidgetPosition = MyGeometry.AbsoluteToLocal(MousePosition) / GraphEditorZoomAmount + GraphEditorLocation;
	
	if (!NewNode->AppendMouseEvent(MyGeometry, MouseEvent, WidgetPosition))
	{
		return false;
	}
	if (UScribbleEdGraphNode* EdGraphNode = ScribbleGraphObj->FindNode(NewNode->GetId()))
	{
		EdGraphNode->SetPosition(NewNode->GetPosition());
		EdGraphNode->SetSize(NewNode->GetSize());
	}
	return true;
}

bool SScribbleGraphPanel::CancelNewNode()
{
	bIsScribbleDragging.Reset();
	if (!NewNode)
	{
		return false;
	}
	ScribbleGraphObj->RemoveScribbleNode(NewNode);
	if (Transaction)
	{
		Transaction->Cancel();
	}
	Transaction.Reset();
	NewNode.Reset();
	return true;
}

bool SScribbleGraphPanel::CommitNewNode()
{
	bIsScribbleDragging.Reset();
	if (!NewNode)
	{
		return false;
	}

	if (!ScribbleGraphObj)
	{
		NewNode.Reset();
		return true;
	}

	const FBox2f Bounds = NewNode->GetContentBounds();
	const FVector2f TopLeft = NewNode->GetPosition() + Bounds.GetCenter() - Bounds.GetExtent();

	NewNode->Modify();
	NewNode->IncrementChangeBracket();
	if (!NewNode->DownSample(ScribblePrecision.Get(3.f)))
	{
		NewNode->DecrementChangeBracket();
		return CancelNewNode();
	}
	
	NewNode->OffsetPosition(TopLeft);
	NewNode->DecrementChangeBracket();
	NewNode.Reset();
	Transaction.Reset();

	return true;
}

SScribbleGraph::~SScribbleGraph()
{
}

void SScribbleGraph::Construct(const FArguments& InArgs)
{
	bIsScribbleEnabled = false;
	
	ChildSlot
	[
		SNew(SOverlay)

		+SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(0)
		[
			SAssignNew(GraphPanel, SScribbleGraphPanel)
			.GraphData(InArgs._GraphData)
			.ScribbleEnabled(this, &SScribbleGraph::IsScribbleEnabled)
			.ScribbleColor(this, &SScribbleGraph::GetScribbleColor)
			.ScribbleThickness(this, &SScribbleGraph::GetScribbleThickness)
			.ScribblePrecision(this, &SScribbleGraph::GetScribblePrecision)
			.ShouldDrawBackground(InArgs._ShouldDrawBackground)
			.ShouldDrawSurroundingShadow(InArgs._ShouldDrawSurroundingShadow)
			.AllowNavigation(InArgs._AllowNavigation)
			.PanelToSync(InArgs._PanelToSync)
		]

		+SOverlay::Slot()
		.HAlign(InArgs._ToolbarHAlignment)
		.VAlign(InArgs._ToolbarVAlignment)
		.Padding(InArgs._ToolbarPadding)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText(LOCTEXT("EnabledTooltip", "Enabled"))
				.OnClicked_Lambda([this]() -> FReply
				{
					bIsScribbleEnabled = !bIsScribbleEnabled;
					return FReply::Handled();
				})
				.ContentPadding(2)
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda([this]()
					{
						if (bIsScribbleEnabled)
						{
							return FStyleColors::AccentBlue;
						}
						return FSlateColor::UseForeground();
					})
					.DesiredSizeOverride(FVector2D(20,20))
					.Image(FAppStyle::Get().GetBrush("MeshPaint.Brush"))
				]
			]
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(8, 0, 0, 0)
			[
				SNew(SButton)
				.Visibility_Lambda([this]()
				{
					return bIsScribbleEnabled ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText(LOCTEXT("LineStripTooltip", "Draw Lines"))
				.OnClicked_Lambda([this]() -> FReply
				{
					if (GraphPanel->GetActiveNodeType() == EScribbleNodeType::Invalid)
					{
						GraphPanel->SetActiveNodeType(EScribbleNodeType::LineStrip);
					}
					else
					{
						GraphPanel->ActivateSelectionMode();
					}
					return FReply::Handled();
				})
				.ContentPadding(2)
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda([this]()
					{
						if (GraphPanel->GetActiveNodeType() == EScribbleNodeType::Invalid)
						{
							return FStyleColors::AccentBlue;
						}
						return FSlateColor::UseForeground();
					})
					.DesiredSizeOverride(FVector2D(20,20))
					.Image(FAppStyle::Get().GetBrush("MeshPaint.SelectVertex"))
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4,0,0,0)
			[
				SNew(SColorBlock)
				.Visibility_Lambda([this]()
				{
					return bIsScribbleEnabled ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.CornerRadius(FVector4(3.f, 3.f, 3.f, 3.f))
				.Color_Lambda([]() -> FLinearColor
				{
					return UScribbleEditorSettings::Get()->Color;
				})
				.OnMouseButtonDown_Lambda([this](const FGeometry&, const FPointerEvent&) -> FReply
				{
					FColorPickerArgs PickerArgs;
					PickerArgs.bIsModal = true;
					PickerArgs.ParentWidget = AsShared();
					PickerArgs.InitialColor = UScribbleEditorSettings::Get()->Color;
					PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateLambda([](FLinearColor NewValue)
					{
						UScribbleEditorSettings::Get()->Color = NewValue;
						UScribbleEditorSettings::Get()->SaveConfig();
					});

					OpenColorPicker(PickerArgs);
					return FReply::Handled();
				})
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0)
			[
				SNew(SButton)
				.Visibility_Lambda([this]()
				{
					if (!bIsScribbleEnabled)
					{
						return EVisibility::Collapsed;
					}
					if (FScribbleGraphData* ScribbleGraph = GraphPanel->GetScribbleGraph())
					{
						if (!ScribbleGraph->SupportsAnchors())
						{
							return EVisibility::Collapsed;
						}
					}
					return EVisibility::Visible;
				})
				.IsEnabled_Lambda([this]
				{
					if (FScribbleGraphData* ScribbleGraph = GraphPanel->GetScribbleGraph())
					{
						if (!ScribbleGraph->GetCurrentAnchor().IsNone())
						{
							return true;
						}
					}
					return false;
				})
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText(LOCTEXT("AnchorTooltip", "Anchor to selected node"))
				.OnClicked_Lambda([this]() -> FReply
				{
					if (UScribbleEdGraph* ScribbleEdGraph = GraphPanel->GetScribbleEdGraph())
					{
						const TArray<FGuid> SelectedIds = ScribbleEdGraph->GetSelectedNodeIds();
						if (!SelectedIds.IsEmpty())
						{
							if (FScribbleGraphData* ScribbleGraph = GraphPanel->GetScribbleGraph())
							{
								const FName& CurrentAnchor = ScribbleGraph->GetCurrentAnchor();

								bool bShouldEnableAnchor = true;
								for (const FGuid& NodeId : SelectedIds)
								{
									if (FScribbleNode* ScribbleNode = ScribbleGraph->FindNode(NodeId))
									{
										if (!ScribbleNode->GetAnchor().IsNone())
										{
											bShouldEnableAnchor = false;
											break;
										}
									}
								}

								ScribbleGraph->Modify();
								for (const FGuid& NodeId : SelectedIds)
								{
									if (FScribbleNode* ScribbleNode = ScribbleGraph->FindNode(NodeId))
									{
										ScribbleNode->SetAnchor(bShouldEnableAnchor ? CurrentAnchor : NAME_None);
									}
								}
							}
						}
					}
						
					return FReply::Handled();
				})
				.ContentPadding(2)
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda([this]()
					{
						if (const UScribbleEdGraph* ScribbleEdGraph = GraphPanel->GetScribbleEdGraph())
						{
							for (const FGuid& Guid : GraphPanel->GetScribbleEdGraph()->GetSelectedNodeIds())
							{
								if (const UScribbleEdGraphNode* ScribbleEdGraphNode = ScribbleEdGraph->FindNode(Guid))
								{
									if (!ScribbleEdGraphNode->GetScribbleNode()->GetAnchor().IsNone())
									{
										return FStyleColors::AccentBlue;
									}
								}
							}
						}
						return FSlateColor::UseForeground();
					})
					.DesiredSizeOverride(FVector2D(20,20))
					.Image(FAppStyle::Get().GetBrush("Icons.Snap"))
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4,0,0,0)
			[
				SNew(SSpinBox<float>)
				.ToolTipText(LOCTEXT("ThicknessTooltip", "Thickness of the brush"))
				.Delta(0.25f)
				.MinSliderValue(3.f)
				.MaxSliderValue(24.f)
				.MinFractionalDigits(2)
				.MaxFractionalDigits(2)
				.PreventThrottling(true)
				.Visibility_Lambda([this]()
				{
					return bIsScribbleEnabled ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.Value_Lambda([]() -> float
				{
					return UScribbleEditorSettings::Get()->Thickness;
				})
				.OnValueChanged_Lambda([](float InNewValue)
				{
					UScribbleEditorSettings::Get()->Thickness = InNewValue;
				})
				.OnValueCommitted_Lambda([](float InNewValue, ETextCommit::Type InCommitType)
				{
					UScribbleEditorSettings::Get()->Thickness = InNewValue;
					UScribbleEditorSettings::Get()->SaveConfig();
				})
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4,0,0,0)
			[
				SNew(SSpinBox<float>)
				.ToolTipText(LOCTEXT("SmoothingTooltip", "The smoothing of the strokes\n0 == maintain curvature\n1 == smooth / straigthen lines"))
				.Delta(0.025f)
				.MinSliderValue(0.0)
				.MaxSliderValue(1.0)
				.MinFractionalDigits(2)
				.MaxFractionalDigits(2)
				.PreventThrottling(true)
				.Visibility_Lambda([this]()
				{
					return bIsScribbleEnabled ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.Value_Lambda([]() -> float
				{
					return UScribbleEditorSettings::Get()->GetSmoothing();
				})
				.OnValueChanged_Lambda([](float InNewValue)
				{
					UScribbleEditorSettings::Get()->SetSmoothing(InNewValue);
				})
				.OnValueCommitted_Lambda([](float InNewValue, ETextCommit::Type InCommitType)
				{
					UScribbleEditorSettings::Get()->SetSmoothing(InNewValue);
					UScribbleEditorSettings::Get()->SaveConfig();
				})
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(16,0,0,0)
			[
				SNew(SButton)
				.Visibility_Lambda([this]()
				{
					if (!bIsScribbleEnabled)
					{
						return EVisibility::Collapsed;
					}
					return EVisibility::Visible;
				})
				.IsEnabled_Lambda([this]
				{
					return GUnrealEd->Trans->CanUndo();
				})
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText(LOCTEXT("Undo", "Undo"))
				.OnClicked_Lambda([this]() -> FReply
				{
					GEditor->UndoTransaction();
					return FReply::Handled();
				})
				.ContentPadding(2)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("GenericCommands.Undo"))
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0)
			[
				SNew(SButton)
				.Visibility_Lambda([this]()
				{
					if (!bIsScribbleEnabled)
					{
						return EVisibility::Collapsed;
					}
					return EVisibility::Visible;
				})
				.IsEnabled_Lambda([this]
				{
					return GUnrealEd->Trans->CanRedo();
				})
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText(LOCTEXT("Redo", "Redo"))
				.OnClicked_Lambda([this]() -> FReply
				{
					GEditor->RedoTransaction();
					return FReply::Handled();
				})
				.ContentPadding(2)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("GenericCommands.Redo"))
				]
			]
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0)
			[
				SNew(SButton)
				.Visibility_Lambda([this]()
				{
					if (!bIsScribbleEnabled)
					{
						return EVisibility::Collapsed;
					}
					return EVisibility::Visible;
				})
				.IsEnabled_Lambda([this]
				{
					return !GraphPanel->GetScribbleEdGraph()->GetSelectedNodeIds().IsEmpty();
				})
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText(LOCTEXT("Delete", "Delete"))
				.OnClicked_Lambda([this]() -> FReply
				{
					GraphPanel->GetScribbleEdGraph()->RemoveSelectedNodes();
					return FReply::Handled();
				})
				.ContentPadding(2)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				]
			]
		]
	];

	SetVisibility(EVisibility::SelfHitTestInvisible);
}

FLinearColor SScribbleGraph::GetScribbleColor() const
{
	return UScribbleEditorSettings::Get()->Color;
}

float SScribbleGraph::GetScribbleThickness() const
{
	return UScribbleEditorSettings::Get()->Thickness;
}

float SScribbleGraph::GetScribblePrecision() const
{
	return UScribbleEditorSettings::Get()->Precision;
}

#undef LOCTEXT_NAMESPACE
