// Copyright Epic Games, Inc. All Rights Reserved.

#include "SScribbleGraphNode.h"
#include "SScribbleGraph.h"

#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "ScribbleEdGraphNode.h"
#include "SGraphPin.h"
#include "SNodePanel.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "ScribbleEdGraph.h"
#include "SGraphPanel.h"

class SWidget;
class UEdGraphSchema;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "RigDependencyGraphEditor"

void SScribbleGraphNode::Construct(const FArguments& InArgs, class UScribbleEdGraphNode* InNode)
{
	GraphNode = InNode;
	UpdateGraphNode();
}

const FSlateBrush* SScribbleGraphNode::GetShadowBrush(bool bSelected) const
{
	return nullptr;
}

void SScribbleGraphNode::UpdateGraphNode()
{
}

FReply SScribbleGraphNode::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (const FScribbleNode* ScribbleNode = GetScribbleNode())
	{
		const FVector2f LocalPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		if (!ScribbleNode->IntersectsCursorPosition(LocalPosition))
		{
		}
	}
	return SGraphNode::OnMouseButtonDown(MyGeometry, MouseEvent);
}

FReply SScribbleGraphNode::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return SGraphNode::OnMouseMove(MyGeometry, MouseEvent);
}

FReply SScribbleGraphNode::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return SGraphNode::OnMouseButtonUp(MyGeometry, MouseEvent);
}

void SScribbleGraphNode::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	if (FScribbleNode* ScribbleNode = GetScribbleNode())
	{
		ScribbleNode->SetPosition(NewPosition);
	}
}

int32 SScribbleGraphNode::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
                                  FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 LastLayerId = SGraphNode::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	if (const UScribbleEdGraphNode* ScribbleEdGraphNode = GetScribbleEdGraphNode())
	{
		if (const FScribbleNode* ScribbleNode = ScribbleEdGraphNode->GetScribbleNode())
		{
			bool bSelected = false;
			if (const UScribbleEdGraph* ScribbleEdGraph = Cast<UScribbleEdGraph>(ScribbleEdGraphNode->GetGraph()))
			{
				bSelected = ScribbleEdGraph->IsNodeSelected(ScribbleEdGraphNode);
			}

			float ZoomAmount = 1.f;
			if (OwnerGraphPanelPtr.IsValid())
			{
				if (TSharedPtr<SScribbleGraphPanel> GraphPanel = StaticCastSharedPtr<SScribbleGraphPanel,SGraphPanel>(OwnerGraphPanelPtr.Pin()))
				{
					ZoomAmount = GraphPanel->GetZoomAmount();

					// don't render any node as selected if the
					// panel is not enabled for scribbling
					if (!GraphPanel->IsScribbleEnabled())
					{
						bSelected = false;
					}
				}
			}
			LastLayerId = ScribbleNode->OnPaint(AllottedGeometry, OutDrawElements, ++LastLayerId, InWidgetStyle, bSelected, ZoomAmount);
		}
	}
	return LastLayerId;
}

FVector2D SScribbleGraphNode::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D(GetSizeFromScribbleNode());
}

bool SScribbleGraphNode::CanBeSelected(const FVector2f& MousePositionInNode) const
{
	if (const FScribbleNode* ScribbleNode = GetScribbleNode())
	{
		if (!ScribbleNode->IntersectsCursorPosition(MousePositionInNode))
		{
			return false;
		}
	}
	return true;
}

const UScribbleEdGraphNode* SScribbleGraphNode::GetScribbleEdGraphNode() const
{
	return Cast<UScribbleEdGraphNode>(GraphNode);
}

UScribbleEdGraphNode* SScribbleGraphNode::GetScribbleEdGraphNode()
{
	return Cast<UScribbleEdGraphNode>(GraphNode);
}

const FScribbleNode* SScribbleGraphNode::GetScribbleNode() const
{
	if (const UScribbleEdGraphNode* EdGraphNode = GetScribbleEdGraphNode())
	{
		return EdGraphNode->GetScribbleNode();
	}
	return nullptr;
}

FScribbleNode* SScribbleGraphNode::GetScribbleNode()
{
	if (UScribbleEdGraphNode* EdGraphNode = GetScribbleEdGraphNode())
	{
		return EdGraphNode->GetScribbleNode();
	}
	return nullptr;
}

const FScribbleGraphData* SScribbleGraphNode::GetScribbleGraphData() const
{
	if (const UScribbleEdGraphNode* EdGraphNode = GetScribbleEdGraphNode())
	{
		return EdGraphNode->GetScribbleGraphData();
	}
	return nullptr;
}

FScribbleGraphData* SScribbleGraphNode::GetScribbleGraphData()
{
	if (UScribbleEdGraphNode* EdGraphNode = GetScribbleEdGraphNode())
	{
		return EdGraphNode->GetScribbleGraphData();
	}
	return nullptr;
}

FVector2f SScribbleGraphNode::GetSizeFromScribbleNode() const
{
	if (const UScribbleEdGraphNode* ScribbleEdGraphNode = GetScribbleEdGraphNode())
	{
		return ScribbleEdGraphNode->GetSize();
	}
	return FVector2f();
}

#undef LOCTEXT_NAMESPACE
