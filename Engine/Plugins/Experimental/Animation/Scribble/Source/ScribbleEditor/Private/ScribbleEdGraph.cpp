// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScribbleEdGraph.h"

#include "ITransportControl.h"
#include "ScribbleEdGraphNode.h"
#include "ScribbleEdGraphPanelNodeFactory.h"
#include "ScribbleEdGraphSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScribbleEdGraph)

UScribbleEdGraph::UScribbleEdGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIsSelecting(false)
{
}

void UScribbleEdGraph::Initialize(const TSharedPtr<FScribbleGraphData>& InScribbleGraphData)
{
	if (!InScribbleGraphData)
	{
		return;
	}

	GraphData = InScribbleGraphData;

	for (TSharedPtr<FScribbleNode> ScribbleNode : *InScribbleGraphData)
	{
		check(ScribbleNode);
		AddScribbleNode(ScribbleNode);
	}
}

const UScribbleEdGraphNode* UScribbleEdGraph::FindNode(const FGuid& InNodeId) const
{
	if (UScribbleEdGraphNode* const* NodePtr = NodeIdLookup.Find(InNodeId))
	{
		return *NodePtr;
	}
	return nullptr;
}

UScribbleEdGraphNode* UScribbleEdGraph::FindNode(const FGuid& InNodeId)
{
	const UScribbleEdGraph* ConstThis = this;
	return const_cast<UScribbleEdGraphNode*>(ConstThis->FindNode(InNodeId));
}

FGuid UScribbleEdGraph::AddScribbleNode(const TSharedPtr<FScribbleNode>& InNode)
{
	if (!GraphData || !InNode)
	{
		return FGuid();
	}

	if (const UScribbleEdGraphNode* ExistingNode = FindNode(InNode->GetId()))
	{
		return ExistingNode->GetNodeId();
	}

	(void)GraphData->AddNode(InNode);
	
	UScribbleEdGraphNode* ScribbleEdGraphNode = NewObject<UScribbleEdGraphNode>(this);
	ScribbleEdGraphNode->Initialize(InNode);
	NodeIdLookup.Add(InNode->GetId(), ScribbleEdGraphNode);

	AddNode(ScribbleEdGraphNode, false, false);
	return InNode->GetId();
}

bool UScribbleEdGraph::RemoveScribbleNode(const TSharedPtr<FScribbleNode>& InNode)
{
	if (!GraphData || !InNode)
	{
		return false;
	}

	if (!GraphData->RemoveNode(InNode))
	{
		return false;
	}
	
	UScribbleEdGraphNode* ScribbleEdGraphNode = FindNode(InNode->GetId());
	NodeIdLookup.Remove(InNode->GetId());
	SelectedNodes.Remove(InNode->GetId());
	if (ScribbleEdGraphNode)
	{
		RemoveNode(ScribbleEdGraphNode);
	}
	
	return true;
}

bool UScribbleEdGraph::RemoveScribbleNode(const FGuid& InGuid)
{
	if (!GraphData)
	{
		return false;
	}
	TSharedPtr<FScribbleNode> Node = GraphData->FindNodePtr(InGuid);
	if (!Node)
	{
		return false;
	}
	return RemoveScribbleNode(Node);
}

bool UScribbleEdGraph::Contains(const FGuid& InNodeId) const
{
	return FindNode(InNodeId) != nullptr;
}

FScribbleGraphData* UScribbleEdGraph::GetGraphData()
{
	return GraphData.Get();
}

const FScribbleGraphData* UScribbleEdGraph::GetGraphData() const
{
	return GraphData.Get();
}

void UScribbleEdGraph::RebuildGraph()
{
	if (!GraphData)
	{
		return;
	}

	// we may want to be lazy here and only remove / add what's necessary
	Nodes.Reset();
	NodeIdLookup.Reset();

	GraphData->IncrementChangeBracket();
	
	for (const TSharedPtr<FScribbleNode>& ScribbleNode : *GraphData)
	{
		check(ScribbleNode);
		AddScribbleNode(ScribbleNode);
	}

	GraphData->DecrementChangeBracket();
	NotifyGraphChanged();
}

void UScribbleEdGraph::SelectAllNodes()
{
	SelectNodes(GetAllNodeIds());
}

void UScribbleEdGraph::ClearSelection()
{
	SelectNodes({});
}

void UScribbleEdGraph::RemoveAllNodes()
{
	if (Nodes.IsEmpty())
	{
		return;
	}

	if (!GraphData)
	{
		return;
	}

	GraphData->IncrementChangeBracket();
	
	TArray<UEdGraphNode*> NodesToRemove = Nodes;
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		if (const UScribbleEdGraphNode* NodeToRemove = Cast<UScribbleEdGraphNode>(NodesToRemove[NodeIndex]))
		{
			RemoveScribbleNode(NodeToRemove->GetNodeId());
		}
	}

	Nodes.Reset();
	
	GraphData->DecrementChangeBracket();
	
	NotifyGraphChanged();
}

TArray<FGuid> UScribbleEdGraph::GetAllNodeIds() const
{
	TArray<FGuid> NodeIds;
	NodeIds.Reserve(Nodes.Num());
	for (const TObjectPtr<UEdGraphNode>& Node : Nodes)
	{
		if (const UScribbleEdGraphNode* ScribbleEdGraphNode = Cast<UScribbleEdGraphNode>(Node))
		{
			NodeIds.Add(ScribbleEdGraphNode->GetNodeId());
		}
	}
	return NodeIds;
}

void UScribbleEdGraph::RemoveSelectedNodes()
{
	if (Nodes.IsEmpty() || SelectedNodes.IsEmpty())
	{
		return;
	}

	if (!GraphData)
	{
		return;
	}

	GraphData->IncrementChangeBracket();

	TArray<TObjectPtr<UEdGraphNode>> NodesToRemove = Nodes;
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		UScribbleEdGraphNode* ScribbleEdGraphNode = Cast<UScribbleEdGraphNode>(NodesToRemove[NodeIndex]);
		if (!ScribbleEdGraphNode)
		{
			continue;
		}
		if (!SelectedNodes.Contains(ScribbleEdGraphNode->GetNodeId()))
		{
			continue;
		}
		RemoveScribbleNode(ScribbleEdGraphNode->GetNodeId());
	}

	SelectedNodes.Reset();
	GraphData->DecrementChangeBracket();
	
	NotifyGraphChanged();
}

bool UScribbleEdGraph::IsNodeSelected(const FGuid& InNodeId) const
{
	return SelectedNodes.Contains(InNodeId);
}

bool UScribbleEdGraph::IsNodeSelected(const TSharedPtr<FScribbleNode>& InNode) const
{
	if (!InNode)
	{
		return false;
	}
	return IsNodeSelected(InNode->GetId());
}

bool UScribbleEdGraph::IsNodeSelected(const UScribbleEdGraphNode* InNode) const
{
	if (!InNode)
	{
		return false;
	}
	return IsNodeSelected(InNode->GetNodeId());
}

void UScribbleEdGraph::GroupSelectedNodes()
{
	if (!GraphData)
	{
		return;
	}
	
	TArray<TSharedPtr<FScribbleNode>> NodesToGroup;
	for (const FGuid& NodeId : SelectedNodes)
	{
		if (const UScribbleEdGraphNode* Node = FindNode(NodeId))
		{
			if (TSharedPtr<FScribbleNode> ScribbleNode = Node->GetScribbleNodePtr())
			{
				NodesToGroup.Add(ScribbleNode);
			}
		}
	}
	
	TSharedPtr<FScribbleNode> GroupedNode = GraphData->GroupNodes(NodesToGroup);
	if (!GroupedNode)
	{
		return;
	}

	for (const TSharedPtr<FScribbleNode>& ScribbleNode : NodesToGroup)
	{
		RemoveScribbleNode(ScribbleNode);
	}

	AddScribbleNode(GroupedNode);
	if (IsAnchorEnabled.Get(false))
	{
		GroupedNode->SetAnchor(GetGraphData()->GetCurrentAnchor());
	}
	NotifyGraphChanged();

	SelectNodes({GroupedNode->GetId()});
}

void UScribbleEdGraph::UngroupSelectedNodes()
{
	if (!GraphData)
	{
		return;
	}

	if (SelectedNodes.Num() != 1)
	{
		return;
	}

	TSharedPtr<FScribbleNode> GroupedNode;
	if (const UScribbleEdGraphNode* Node = FindNode(SelectedNodes[0]))
	{
		if (TSharedPtr<FScribbleNode> ScribbleNode = Node->GetScribbleNodePtr())
		{
			GroupedNode = ScribbleNode;
		}
	}

	TArray<TSharedPtr<FScribbleNode>> UngroupedNodes = GraphData->UngroupNode(GroupedNode);
	if (UngroupedNodes.IsEmpty())
	{
		return;
	}

	RemoveScribbleNode(GroupedNode);

	TArray<FGuid> NodesToSelect;
	for (const TSharedPtr<FScribbleNode>& UngroupedNode : UngroupedNodes)
	{
		AddScribbleNode(UngroupedNode);
		NodesToSelect.Add(UngroupedNode->GetId());
	}
	NotifyGraphChanged();

	SelectNodes(NodesToSelect);
}

void UScribbleEdGraph::NotifyGraphChanged()
{
	if (GraphData)
	{
		if (GraphData->ChangeBracket > 0)
		{
			return;
		}
	}
	Super::NotifyGraphChanged();
}

FBox2D UScribbleEdGraph::GetAllNodesBounds() const
{
	if (Nodes.IsEmpty())
	{
		return FBox2D();
	}

	FBox2D Bounds(EForceInit::ForceInit);
	for (const TObjectPtr<UEdGraphNode>& Node : Nodes)
	{
		const FVector2D TopLeft(Node->NodePosX, Node->NodePosY);
		Bounds += TopLeft;
		Bounds += TopLeft + FVector2D(Node->GetSize());
	}
	return Bounds;
}

FBox2D UScribbleEdGraph::GetSelectedNodesBounds() const
{
	if (Nodes.IsEmpty() || SelectedNodes.IsEmpty())
	{
		return FBox2D();
	}

	return GetNodesBounds(SelectedNodes);
}

void UScribbleEdGraph::SelectNodes(const TArray<FGuid>& InNodeIds)
{
	SelectedNodes.Reset();
	
	TSet<const UEdGraphNode*> NodesToSelect;
	for (const FGuid& NodeId : InNodeIds)
	{
		if (const UScribbleEdGraphNode* Node = FindNode(NodeId))
		{
			SelectedNodes.Add(NodeId);
			NodesToSelect.Add(Node);
		}
	}

	if (NodesToSelect.IsEmpty())
	{
		return;
	}

	if (bIsSelecting)
	{
		return;
	}
	const TGuardValue<bool> _(bIsSelecting, true);
	SelectNodeSet(NodesToSelect);
}
