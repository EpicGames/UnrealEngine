// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScribbleEdGraphCommands.h"

#define LOCTEXT_NAMESPACE "ScribbleEdGraphCommands"

void FScribbleEdGraphCommands::RegisterCommands()
{
	UI_COMMAND(FrameSelection, "Frame Selection", "Expands and frames the selection graph", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
	UI_COMMAND(DeleteSelection, "Delete Selection", "Delete the selected elements from the graph", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
	UI_COMMAND(SelectAll, "Select All", "Select all scribble elements", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::A));
	UI_COMMAND(GroupSelection, "Group Selection", "Groups the selected elements in the graph", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::G));
	UI_COMMAND(UngroupSelection, "Ungroup Selection", "Ungroups the selected elements in the graph", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::G));
	UI_COMMAND(SelectionTool, "Selection Tool", "Enables / Disables the Selection Tool", EUserInterfaceActionType::Button, FInputChord(EKeys::V));
	UI_COMMAND(BrushTool, "Brush Tool", "Enables / Disables the Brush Tool", EUserInterfaceActionType::Button, FInputChord(EKeys::B));
}

#undef LOCTEXT_NAMESPACE
