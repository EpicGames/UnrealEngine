// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScribbleEdGraphNode.h"
#include "ScribbleEdGraph.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScribbleEdGraphNode)

class UObject;

UScribbleEdGraphNode::UScribbleEdGraphNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UScribbleEdGraphNode::~UScribbleEdGraphNode()
{
	FScribbleNode* ScribbleNode = GetScribbleNode();
	if(!ScribbleNode)
	{
		return;
	}
	ScribbleNode->OnPositionChanged().Remove(PositionChangedDelegateHandle);
	ScribbleNode->OnSizeChanged().Remove(SizeChangedDelegateHandle);
}

UScribbleEdGraph* UScribbleEdGraphNode::GetScribbleEdGraph() const
{
	return CastChecked<UScribbleEdGraph>(GetOuter());
}

void UScribbleEdGraphNode::Initialize(const TSharedPtr<FScribbleNode>& InScribbleNode)
{
	if (!InScribbleNode)
	{
		return;
	}
	WeakScribbleNode = InScribbleNode;
	
	NodePosX = InScribbleNode->GetPosition().X;
	NodePosY = InScribbleNode->GetPosition().Y;
	NodeWidth = InScribbleNode->GetSize().X;
	NodeHeight = InScribbleNode->GetSize().Y;

	TWeakObjectPtr<UScribbleEdGraphNode> WeakThis = this;
	PositionChangedDelegateHandle = InScribbleNode->OnPositionChanged().AddLambda([WeakThis](const FVector2f& InNewPosition)
	{
		if(!WeakThis.IsValid())
		{
			return;
		}
		UScribbleEdGraphNode* StrongThis = WeakThis.Get();
		StrongThis->NodePosX = InNewPosition.X;
		StrongThis->NodePosY = InNewPosition.Y;
	});

	SizeChangedDelegateHandle = InScribbleNode->OnSizeChanged().AddLambda([WeakThis](const FVector2f& InNewSize)
	{
		if(!WeakThis.IsValid())
		{
			return;
		}
		UScribbleEdGraphNode* StrongThis = WeakThis.Get();
		StrongThis->NodeWidth = InNewSize.X;
		StrongThis->NodeHeight = InNewSize.Y;
	});
}

const FScribbleNode* UScribbleEdGraphNode::GetScribbleNode() const
{
	if (WeakScribbleNode.IsValid())
	{
		return WeakScribbleNode.Pin().Get();
	}
	return nullptr;
}

FScribbleNode* UScribbleEdGraphNode::GetScribbleNode()
{
	if (WeakScribbleNode.IsValid())
	{
		return WeakScribbleNode.Pin().Get();
	}
	return nullptr;
}

TSharedPtr<FScribbleNode> UScribbleEdGraphNode::GetScribbleNodePtr() const
{
	if (WeakScribbleNode.IsValid())
	{
		return WeakScribbleNode.Pin();
	}
	return nullptr;
}

const FScribbleGraphData* UScribbleEdGraphNode::GetScribbleGraphData() const
{
	if (const FScribbleNode* ScribbleNode = GetScribbleNode())
	{
		return ScribbleNode->GetGraph(); 
	}
	return nullptr;
}

FScribbleGraphData* UScribbleEdGraphNode::GetScribbleGraphData()
{
	if (FScribbleNode* ScribbleNode = GetScribbleNode())
	{
		return ScribbleNode->GetGraph(); 
	}
	return nullptr;
}

const FGuid& UScribbleEdGraphNode::GetNodeId() const
{
	if (const FScribbleNode* ScribbleNode = GetScribbleNode())
	{
		return ScribbleNode->GetId();
	}
	static constexpr FGuid InvalidGuid = FGuid();
	return InvalidGuid;
}
