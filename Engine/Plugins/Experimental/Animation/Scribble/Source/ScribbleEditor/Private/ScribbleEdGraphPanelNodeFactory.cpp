// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScribbleEdGraphPanelNodeFactory.h"

#include "EdGraph/EdGraphNode.h"
#include "ScribbleEdGraphNode.h"
#include "SScribbleGraphNode.h"
#include "SGraphNode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

TSharedPtr<SGraphNode> FScribbleEdGraphPanelNodeFactory::CreateNode(UEdGraphNode* Node) const
{
	if (UScribbleEdGraphNode* ScribbleEdGraphNode = Cast<UScribbleEdGraphNode>(Node))
	{
		TSharedRef<SGraphNode> GraphNode = SNew(SScribbleGraphNode, ScribbleEdGraphNode);
		//GraphNode->SlatePrepass();
		//ScribbleEdGraphNode->SetDimensions(GraphNode->GetDesiredSize());
		return GraphNode;
	}
	return nullptr;
}
