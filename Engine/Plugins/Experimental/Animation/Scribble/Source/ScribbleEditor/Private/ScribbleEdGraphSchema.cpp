// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScribbleEdGraphSchema.h"

#include "ScribbleEdGraphNode.h"
#include "ScribbleEdGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScribbleEdGraphSchema)

#define LOCTEXT_NAMESPACE "ScribbleEdGraphSchema"

UScribbleEdGraphSchema::UScribbleEdGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UScribbleEdGraphNode* UScribbleEdGraphSchema::CreateGraphNode(UScribbleEdGraph* InGraph, const FGuid& InNodeId) const
{
	if (!InGraph)
	{
		return nullptr;
	}

	const FScribbleGraphData* GraphData = InGraph->GetGraphData();
	if (!GraphData)
	{
		return nullptr;
	}
	
	static constexpr bool bSelectNewNode = false;
	
	if (TSharedPtr<FScribbleNode> ScribbleNode = GraphData->FindNodePtr(InNodeId))
	{
		FGraphNodeCreator<UScribbleEdGraphNode> GraphNodeCreator(*InGraph);
		UScribbleEdGraphNode* ScribbleEdGraphNode = GraphNodeCreator.CreateNode(bSelectNewNode);
		ScribbleEdGraphNode->Initialize(ScribbleNode);
		GraphNodeCreator.Finalize();

		InGraph->NodeIdLookup.Add(InNodeId, ScribbleEdGraphNode);

		return ScribbleEdGraphNode;
	}

	return nullptr;
}

const FPinConnectionResponse UScribbleEdGraphSchema::CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const
{
	FPinConnectionResponse Response;
	Response.Response = CONNECT_RESPONSE_DISALLOW;
	return Response;
}

void UScribbleEdGraphSchema::SetNodePosition(UEdGraphNode* InNode, const FVector2f& InPosition) const
{
	Super::SetNodePosition(InNode, InPosition);

	if (UScribbleEdGraphNode* ScribbleEdGraphNode = Cast<UScribbleEdGraphNode>(InNode))
	{
		if (FScribbleNode* ScribbleNode = ScribbleEdGraphNode->GetScribbleNode())
		{
			ScribbleNode->SetPosition(InPosition);
		}
	}
}

void UScribbleEdGraphSchema::SetNodeSize(UEdGraphNode* InNode, const FVector2f& InSize) const
{
	if (UScribbleEdGraphNode* ScribbleEdGraphNode = Cast<UScribbleEdGraphNode>(InNode))
	{
		if (FScribbleNode* ScribbleNode = ScribbleEdGraphNode->GetScribbleNode())
		{
			ScribbleNode->SetSize(InSize);
		}
	}
}

#undef LOCTEXT_NAMESPACE
