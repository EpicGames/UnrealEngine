// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "ScribbleEdGraphCommands.h"
#include "ScribbleEdGraphPanelNodeFactory.h"

class FScribbleEditorModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		ScribbleEdGraphPanelNodeFactory = MakeShareable(new FScribbleEdGraphPanelNodeFactory());
		FEdGraphUtilities::RegisterVisualNodeFactory(ScribbleEdGraphPanelNodeFactory);
		FScribbleEdGraphCommands::Register();
	}
	
	virtual void ShutdownModule() override
	{
		FEdGraphUtilities::UnregisterVisualNodeFactory(ScribbleEdGraphPanelNodeFactory);
		FScribbleEdGraphCommands::Unregister();
	}

private:
	TSharedPtr<FScribbleEdGraphPanelNodeFactory> ScribbleEdGraphPanelNodeFactory;
};

IMPLEMENT_MODULE(FScribbleEditorModule, ScribbleEditor)
