// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPanel.h"
#include "EditorUndoClient.h"
#include "ScribbleEdGraphNode.h"
#include "ScopedTransaction.h"
#include "Brushes/SlateRoundedBoxBrush.h"

#define UE_API SCRIBBLEEDITOR_API

DECLARE_DELEGATE_OneParam(FScribbleSetNodeType, EScribbleNodeType::Type);

class SScribbleGraphPanel : public SGraphPanel, public FEditorUndoClient
{
public:

	SLATE_BEGIN_ARGS(SScribbleGraphPanel)
		: _ScribbleEnabled(true)
		, _ScribbleNodeType(EScribbleNodeType::Invalid)
		, _ScribbleColor()
		, _ScribbleThickness()
		, _ScribblePrecision()
		, _ShouldDrawBackground(false)
		, _ShouldDrawSurroundingShadow(false)
		, _AllowNavigation(true)
	{}
	SLATE_ARGUMENT(TSharedPtr<FScribbleGraphData>, GraphData)
	SLATE_ATTRIBUTE(bool, ScribbleEnabled)
	SLATE_ATTRIBUTE(EScribbleNodeType::Type, ScribbleNodeType)
	SLATE_EVENT(FScribbleSetNodeType, OnSetNodeType)
	SLATE_ATTRIBUTE(FLinearColor, ScribbleColor)
	SLATE_ATTRIBUTE(float, ScribbleThickness)
	SLATE_ATTRIBUTE(float, ScribblePrecision)
	SLATE_ATTRIBUTE(bool, ShouldDrawBackground)
	SLATE_ATTRIBUTE(bool, ShouldDrawSurroundingShadow)
	SLATE_ARGUMENT(bool, AllowNavigation)
	SLATE_ARGUMENT(TWeakPtr<SGraphPanel>, PanelToSync)
	SLATE_END_ARGS()

	SScribbleGraphPanel();

	UE_API virtual ~SScribbleGraphPanel() override;

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

	UE_API bool IsScribbleEnabled() const;
	UE_API bool HasActiveNodeType() const;
	UE_API void ActivateSelectionMode();
	UE_API EScribbleNodeType::Type GetActiveNodeType() const;
	UE_API void SetActiveNodeType(EScribbleNodeType::Type InNodeType);

	UE_API UScribbleEdGraph* GetScribbleEdGraph();
	UE_API const UScribbleEdGraph* GetScribbleEdGraph() const;
	UE_API FScribbleGraphData* GetScribbleGraph();
	UE_API const FScribbleGraphData* GetScribbleGraph() const;

	UE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	UE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/** FEditorUndoClient interface */
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;

private:

	void BindCommands();
	void UpdateScribbleVisibility();
	void HandleSelectionChanged(const FGraphPanelSelectionSet& SelectionSet);

	bool UpdateNewNode( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );
	bool CancelNewNode();
	bool CommitNewNode();
	
	UScribbleEdGraph* ScribbleGraphObj;
	TSharedPtr<FUICommandList> CommandList;

	TAttribute<bool> ScribbleEnabled;
	TAttribute<EScribbleNodeType::Type> ScribbleNodeType;
	TAttribute<float> ScribbleThickness;
	TAttribute<float> ScribblePrecision;
	TAttribute<FLinearColor> ScribbleColor;
	FScribbleSetNodeType OnSetNodeType;

	TOptional<bool> bIsScribbleDragging;
	TSharedPtr<FScribbleNode> NewNode;
	TSharedPtr<FScopedTransaction> Transaction;
	bool bAllowNavigation;
	TWeakPtr<SGraphPanel> WeakPanelToSync;

	const TUniquePtr<FSlateRoundedBoxBrush> RoundedFrameBrush;
};

class SScribbleGraph : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SScribbleGraph)
		: _ShouldDrawBackground(true)
		, _ShouldDrawSurroundingShadow(false)
		, _AllowNavigation(true)
		, _ToolbarHAlignment(HAlign_Left)
		, _ToolbarVAlignment(VAlign_Top)
		, _ToolbarPadding(20)
	{}
	SLATE_ARGUMENT(TSharedPtr<FScribbleGraphData>, GraphData)
	SLATE_ATTRIBUTE(bool, ShouldDrawBackground)
	SLATE_ATTRIBUTE(bool, ShouldDrawSurroundingShadow)
	SLATE_ARGUMENT(bool, AllowNavigation)
	SLATE_ARGUMENT(EHorizontalAlignment, ToolbarHAlignment)
	SLATE_ARGUMENT(EVerticalAlignment, ToolbarVAlignment)
	SLATE_ARGUMENT(FMargin, ToolbarPadding)
	SLATE_ARGUMENT(TWeakPtr<SGraphPanel>, PanelToSync)
	SLATE_END_ARGS()

	SScribbleGraph()
		: bIsScribbleEnabled(true)
	{
	}

	UE_API virtual ~SScribbleGraph() override;

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

protected:

	FLinearColor GetScribbleColor() const;
	float GetScribbleThickness() const;
	float GetScribblePrecision() const;
	bool IsScribbleEnabled() const { return bIsScribbleEnabled; }

	TSharedPtr<SScribbleGraphPanel> GraphPanel;
	bool bIsScribbleEnabled;
};

#undef UE_API
