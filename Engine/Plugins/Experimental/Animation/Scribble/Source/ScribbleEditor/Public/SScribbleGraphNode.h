// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScribbleGraph.h"
#include "Internationalization/Text.h"
#include "SGraphNode.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API SCRIBBLEEDITOR_API

class SGraphPin;
class SVerticalBox;
class SWidget;
struct FSlateBrush;

class SScribbleGraphNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SScribbleGraphNode){}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs, class UScribbleEdGraphNode* InNode);

protected:
	
	// SGraphNode interface
	virtual void CreatePinWidgets() override {}
	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override {}

	virtual FText GetNodeTooltip() const override { return FText(); }
	virtual FSlateColor GetNodeBodyColor() const override { return FLinearColor(0,0,0,0); }
	virtual const FSlateBrush* GetNodeBodyBrush() const override { return nullptr; }
	UE_API virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	UE_API virtual void UpdateGraphNode() override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	virtual FVector2f GetDesiredSizeForMarquee2f() const override { return GetSizeFromScribbleNode(); }
	UE_API virtual bool CanBeSelected(const FVector2f& MousePositionInNode) const override;

protected:

	const UScribbleEdGraphNode* GetScribbleEdGraphNode() const;
	UScribbleEdGraphNode* GetScribbleEdGraphNode();
	const FScribbleNode* GetScribbleNode() const;
	FScribbleNode* GetScribbleNode();
	const FScribbleGraphData* GetScribbleGraphData() const;
	FScribbleGraphData* GetScribbleGraphData();

	FVector2f GetSizeFromScribbleNode() const;

	friend class SScribbleGraphPanel;
};

#undef UE_API
