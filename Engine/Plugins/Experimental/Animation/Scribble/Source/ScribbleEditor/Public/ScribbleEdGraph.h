// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScribbleEdGraphNode.h"
#include "EdGraph/EdGraph.h"
#include "ScribbleGraph.h"
#include "ScribbleEdGraph.generated.h"

class UScribbleEdGraphSchema;

/**
 * The UEdGraph implementation of a scribble graph.
 * This is used for the ed graph side of things only and corresponds with
 * the FScribbleGraphData as well as the SScribbleGraph.
 */
UCLASS()
class UScribbleEdGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

public:
	void Initialize(const TSharedPtr<FScribbleGraphData>& InScribbleGraphData);

	FScribbleGraphData* GetGraphData();
	const FScribbleGraphData* GetGraphData() const;

	/** Rebuilds graph from selection */
	void RebuildGraph();

	/** Select the nodes */
	void SelectNodes(const TArray<FGuid>& InNodeIds);

	/** Get the scribble graph nodes we are displaying */
	const TArray<TObjectPtr<UEdGraphNode>>& GetNodes() const { return Nodes; }

	/** Returns the node given a Node Id */
	const UScribbleEdGraphNode* FindNode(const FGuid& InNodeId) const;
	UScribbleEdGraphNode* FindNode(const FGuid& InNodeId);

	FGuid AddScribbleNode(const TSharedPtr<FScribbleNode>& InNode);
	bool RemoveScribbleNode(const TSharedPtr<FScribbleNode>& InNode);
	bool RemoveScribbleNode(const FGuid& InGuid);

	/** Returns true if the node given a Node Id exists */
	bool Contains(const FGuid& InNodeId) const;

	void SelectAllNodes();
	void ClearSelection();
	void RemoveAllNodes();
	TArray<FGuid> GetAllNodeIds() const;
	const TArray<FGuid>& GetSelectedNodeIds() const { return SelectedNodes; }
	void RemoveSelectedNodes();
	bool IsNodeSelected(const FGuid& InNodeId) const;
	bool IsNodeSelected(const TSharedPtr<FScribbleNode>& InNode) const;
	bool IsNodeSelected(const UScribbleEdGraphNode* InNode) const;

	void GroupSelectedNodes();
	void UngroupSelectedNodes();

	virtual void NotifyGraphChanged() override;

private:
	FBox2D GetAllNodesBounds() const;
	FBox2D GetSelectedNodesBounds() const;

	template<typename RangeType>
	FBox2D GetNodesBounds(const RangeType& InNodeIds) const
	{
		FBox2D Bounds(EForceInit::ForceInit);
		for (const FGuid& NodeId : InNodeIds)
		{
			if (const UScribbleEdGraphNode* Node = FindNode(NodeId))
			{
				const FVector2D TopLeft(Node->NodePosX, Node->NodePosY);
				Bounds += TopLeft;
				Bounds += TopLeft + FVector2D(Node->GetSize());
			}
		}
		return Bounds;
	}

private:

	TArray<FGuid> SelectedNodes;
	bool bIsSelecting;

	TMap<FGuid, UScribbleEdGraphNode*> NodeIdLookup;

	TSharedPtr<FScribbleGraphData> GraphData;
	TAttribute<bool> IsAnchorEnabled;

	friend class SScribbleGraph;
	friend class SScribbleGraphPanel;
	friend class UScribbleEdGraphSchema;
};

