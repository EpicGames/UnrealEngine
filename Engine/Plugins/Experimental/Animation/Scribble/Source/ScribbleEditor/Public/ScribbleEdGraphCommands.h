// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FScribbleEdGraphCommands : public TCommands<FScribbleEdGraphCommands>
{
public:
	FScribbleEdGraphCommands() : TCommands<FScribbleEdGraphCommands>
	(
		"ScribbleEdGraph",
		NSLOCTEXT("Contexts", "ScribbleEdGraph", "Scribble"),
		NAME_None, // "MainFrame" // @todo Fix this crash
		FAppStyle::Get().GetStyleSetName() // Icon Style Set
	)
	{}

	/* frames the selection in the graph */
	TSharedPtr<FUICommandInfo> FrameSelection;

	/* delete the selected nodes in the graph */
	TSharedPtr<FUICommandInfo> DeleteSelection;

	/* select all scribble elements */
	TSharedPtr<FUICommandInfo> SelectAll;

	/* groups the selection in the graph */
	TSharedPtr<FUICommandInfo> GroupSelection;

	/* ungroups the selection in the graph */
	TSharedPtr<FUICommandInfo> UngroupSelection;

	/* enables / disables the select tool */
	TSharedPtr<FUICommandInfo> SelectionTool;

	/* enables / disables the brush tool */
	TSharedPtr<FUICommandInfo> BrushTool;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
