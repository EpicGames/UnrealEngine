// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNode.h"
#include "ScribbleNode.h"
#include "Textures/SlateIcon.h"
#include "ScribbleEdGraphNode.generated.h"

class UScribbleEdGraph;
class UEdGraphPin;

/**
 * The UEdGraphNode implementation of a scribble graph node.
 * This is used for the ed graph side of things only and corresponds with
 * the FScribbleNode as well as the SScribbleGraphNode.
 */
UCLASS()
class UScribbleEdGraphNode : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	virtual ~UScribbleEdGraphNode();

	class UScribbleEdGraph* GetScribbleEdGraph() const;

	void Initialize(const TSharedPtr<FScribbleNode>& InScribbleNode);

	// UEdGraphNode implementation
	virtual void AllocateDefaultPins() override {}
	virtual bool ShowPaletteIconOnNode() const override { return false; }
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override { return FSlateIcon(); }

	const FScribbleNode* GetScribbleNode() const;
	FScribbleNode* GetScribbleNode();
	TSharedPtr<FScribbleNode> GetScribbleNodePtr() const;
	const FScribbleGraphData* GetScribbleGraphData() const;
	FScribbleGraphData* GetScribbleGraphData();

	const FGuid& GetNodeId() const;

protected:
	
	TWeakPtr<FScribbleNode> WeakScribbleNode;
	FDelegateHandle PositionChangedDelegateHandle;
	FDelegateHandle SizeChangedDelegateHandle;

	friend class UScribbleEdGraph;
	friend class UScribbleEdGraphSchema;
};


