// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"
#include "Templates/SharedPointer.h"

class UEdGraphNode;

#define UE_API SCRIBBLEEDITOR_API

class FScribbleEdGraphPanelNodeFactory : public FGraphPanelNodeFactory
{
	UE_API virtual TSharedPtr<class SGraphNode> CreateNode(UEdGraphNode* Node) const override;
};

#undef UE_API
