// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScribbleEdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "ScribbleEdGraphNode.h"
#include "ScribbleEdGraphSchema.generated.h"

class UScribbleEdGraph;
class UScribbleEdGraphNode;

UCLASS()
class UScribbleEdGraphSchema : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphSchema interface
	virtual FName GetParentContextMenuName() const override { return NAME_None; }
	virtual bool ShouldAlwaysPurgeOnModification() const override { return false; }
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual void SetNodePosition(UEdGraphNode* InNode, const FVector2f& InPosition) const override;
	virtual void SetNodeSize(UEdGraphNode* InNode, const FVector2f& InSize) const;
	// End of UEdGraphSchema interface

	virtual UScribbleEdGraphNode* CreateGraphNode(UScribbleEdGraph* InGraph, const FGuid& InNodeId) const;
};

